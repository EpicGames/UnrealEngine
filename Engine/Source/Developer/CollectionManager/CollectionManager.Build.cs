// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CollectionManager : ModuleRules
{
	public CollectionManager(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DirectoryWatcher",
				"Analytics",
				"SourceControl",
				"DeveloperSettings"
			}
			);
	}
}
