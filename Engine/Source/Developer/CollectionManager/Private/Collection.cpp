// Copyright Epic Games, Inc. All Rights Reserved.

#include "Collection.h"
#include "CollectionSettings.h"
#include "HAL/PlatformTime.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "CollectionManagerLog.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/ScopeRWLock.h"
#include "Async/ParallelFor.h"
#include "String/ParseLines.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#define LOCTEXT_NAMESPACE "CollectionManager"

static TAutoConsoleVariable<int32> CVarCollectionsMaxCLDescriptionPathCount(
	TEXT("Collections.MaxCLDescriptionPathCount"),
	1000,
	TEXT("Sets the maximum number of paths reported in a changelist when checking in a collection that adds or removes entries."),
	ECVF_Default);

struct FCollectionUtils
{
	static void AppendCollectionToArray(const TSet<FSoftObjectPath>& InObjectSet, TArray<FSoftObjectPath>& OutObjectArray)
	{
		OutObjectArray.Reserve(OutObjectArray.Num() + InObjectSet.Num());
		for (const FSoftObjectPath& ObjectName : InObjectSet)
		{
			OutObjectArray.Add(ObjectName);
		}
	}
};

FCollection::FCollection(const FString& InFilename, bool InUseSCC, ECollectionStorageMode::Type InStorageMode)
{
	ensure(InFilename.Len() > 0);

	bUseSCC = InUseSCC;
	SourceFilename = InFilename;
	CollectionName = FName(*FPaths::GetBaseFilename(InFilename));

	StorageMode = InStorageMode;

	CollectionGuid = FGuid::NewGuid();

	// Initialize the file version to the most recent
	FileVersion = ECollectionVersion::CurrentVersion;
}

TSharedRef<FCollection> FCollection::Clone(const FString& InFilename, bool InUseSCC, ECollectionCloneMode InCloneMode) const
{
	TSharedRef<FCollection> NewCollection = MakeShareable(new FCollection(*this));

	// Set the new collection name and path
	NewCollection->bUseSCC = InUseSCC;
	NewCollection->SourceFilename = InFilename;
	NewCollection->CollectionName = FName(*FPaths::GetBaseFilename(InFilename));

	NewCollection->StorageMode = StorageMode;

	// Create a new GUID?
	if (InCloneMode == ECollectionCloneMode::Unique)
	{
		NewCollection->CollectionGuid = FGuid::NewGuid();
	}

	return NewCollection;
}

static bool ParseUtf8Bom(FMutableMemoryView& Buffer)
{
	const uint64 Size = Buffer.GetSize();
	const uint8* const Bytes = static_cast<const uint8*>(Buffer.GetData());
	if (!(Size & 1) && Size >= 2 && Bytes[0] == 0xff && Bytes[1] == 0xfe)
	{
		// Unicode Intel byte order BOM, leave it to be consumed by FFileHelper::BufferToString
		return false;
	}
	if (!(Size & 1) && Size >= 2 && Bytes[0] == 0xfe && Bytes[1] == 0xff)
	{
		// Unicode non-Intel byte order BOM, leave it to be consumed by FFileHelper::BufferToString
		return false;
	}
	if (Size >= 3 && Bytes[0] == 0xef && Bytes[1] == 0xbb && Bytes[2] == 0xbf)
	{
		// Utf8 BOM, consume it and treat buffer as Utf8 without BOM
		Buffer += 3;
	}
	// Assume Utf8 without BOM
	return true;
}

bool FCollection::Load(FText& OutError)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCollection::Load)
	Empty();

	TArray<uint8> Buffer;
	if (!FFileHelper::LoadFileToArray(Buffer, *SourceFilename))
	{
		OutError = FText::Format(LOCTEXT("LoadError_FailedToLoadFile", "Failed to load the collection '{0}' from disk."), FText::FromString(SourceFilename));
		return false;
	}

	FMutableMemoryView BufferView(Buffer.GetData(), Buffer.Num());
	if (ParseUtf8Bom(BufferView))
	{
		FUtf8StringView FullFileContentsString(
			reinterpret_cast<const UTF8CHAR*>(BufferView.GetData()), IntCastChecked<int32>(BufferView.GetSize()));
		return ParseCollection(FullFileContentsString, OutError);
	}
	else
	{
		FString FullFileContentsString;
		FFileHelper::BufferToString(FullFileContentsString, Buffer.GetData(), Buffer.Num());
		Buffer.Reset();
		return ParseCollection(FStringView(FullFileContentsString), OutError);
	}
}

template <typename CharType>
bool FCollection::ParseCollection(TStringView<CharType> FullFileContentsString, FText& OutError)
{
	// Reserve an initial estimate and parse all lines
	TArray<TStringView<CharType>> FileContents;
	FileContents.Reserve(FullFileContentsString.Len() / 64);
	UE::String::ParseLines(FullFileContentsString, [&FileContents](const TStringView<CharType>& Line) { FileContents.Add(Line); });

	if (FileContents.Num() == 0)
	{
		// Empty file, assume static collection with no items
		return true;
	}

	// Load the header from the contents array
	TMap<FString, FString> HeaderPairs;
	
	int32 LineIndex = 0;
	for (int32 Num = FileContents.Num(); LineIndex < Num; ++LineIndex)
	{
		TStringView<CharType> Line(FileContents[LineIndex]);
		Line.TrimStartAndEndInline();

		if (Line.Len() == 0)
		{
			// Empty line. Done reading headers.
			++LineIndex;
			break;
		}

		int32 Offset;
		if (Line.FindChar(CharType(':'), Offset))
		{
			FString Key(Line.Left(Offset));
			FString Value(Line.Right(Line.Len() - Offset - 1));
			HeaderPairs.Emplace(MoveTemp(Key), MoveTemp(Value));
		}
	}

	// Now process the header pairs to prepare and validate this collection
	if ( !LoadHeaderPairs(HeaderPairs) )
	{
		// Bad header
		OutError = FText::Format(LOCTEXT("LoadError_BadHeader", "The collection file '{0}' contains a bad header and could not be loaded."), FText::FromString(SourceFilename));
		return false;
	}

	// Now load the content if the header load was successful
	if (StorageMode == ECollectionStorageMode::Static)
	{
		if (const int32 NamesNum = FileContents.Num() - LineIndex; NamesNum > 0)
		{
			struct FItem
			{
				FSoftObjectPath Path;
				int32 Hash;
			};

			TArray<FItem> Paths;
			Paths.SetNum(NamesNum);

			typedef decltype(ObjectSet)::KeyFuncsType KeyFuncsType;
			// Name hashing to register new FName takes time
			// Process as much as possible in multiple threads
			TStringView<CharType>* Lines = &FileContents[LineIndex];
			ParallelFor(
				NamesNum,
				[&Lines, &Paths](int32 LocalLineIndex)
			{
				Paths[LocalLineIndex].Path = FSoftObjectPath::ConstructFromStringPath(Lines[LocalLineIndex].TrimStartAndEnd());
				Paths[LocalLineIndex].Hash = KeyFuncsType::GetKeyHash(KeyFuncsType::GetSetKey(Paths[LocalLineIndex].Path));
			},
				// Do not pay for scheduling cost if number of items is too low
				NamesNum < 1000 ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None
			);

			ObjectSet.Reserve(ObjectSet.Num() + Paths.Num());
			// Static collection, a flat list of asset paths
			for (FItem& Item : Paths)
			{
				if (Item.Path.IsNull())
				{
					continue;
				}

				ObjectSet.EmplaceByHash(Item.Hash, MoveTemp(Item.Path));
			}
		}
	}
	else
	{
		// Dynamic collection, a single query line
		DynamicQueryText = (FileContents.Num() > LineIndex) ? FString(FileContents[LineIndex].TrimStartAndEnd()) : FString();
	}

	DiskSnapshot.TakeSnapshot(*this);
	bChangedSinceLastDiskSnapshot = false;

	return true;
}

bool FCollection::Save(const TArray<FText>& AdditionalChangelistText, FText& OutError, bool bForceCommitToRevisionControl)
{
	if ( !ensure(SourceFilename.Len()) )
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		return false;
	}

	// Store the start time for profiling reasons
	double SaveStartTime = FPlatformTime::Seconds();

	// Keep track of save progress to update the slow task dialog
	const int32 SaveProgressDenominator = 3;
	int32 SaveProgressNumerator = 0;


	GWarn->BeginSlowTask( FText::Format( LOCTEXT("SavingCollection", "Saving Collection {0}"), FText::FromName( CollectionName ) ), true);
	GWarn->UpdateProgress(SaveProgressNumerator++, SaveProgressDenominator);

	if ( bUseSCC )
	{
		// Checkout the file
		if ( !CheckoutCollection(OutError) )
		{
			UE_LOGF(LogCollectionManager, Error, "Failed to check out a collection file: %ls", *CollectionName.ToString());
			GWarn->EndSlowTask();
			return false;
		}
	}

	GWarn->UpdateProgress(SaveProgressNumerator++, SaveProgressDenominator);

	// Generate a string with the file contents
	FString FileOutput;

	// Start with the header
	TMap<FString,FString> HeaderPairs;
	SaveHeaderPairs(HeaderPairs);
	for (const auto& HeaderPair : HeaderPairs)
	{
		FileOutput += HeaderPair.Key + TEXT(":") + HeaderPair.Value + LINE_TERMINATOR;
	}
	FileOutput += LINE_TERMINATOR;

	// Now for the content
	if (StorageMode == ECollectionStorageMode::Static)
	{
		// Write out the set as a sorted array to keep things in a known order for diffing
		TArray<FSoftObjectPath> ObjectList = ObjectSet.Array();
		ObjectList.Sort([](FSoftObjectPath A, FSoftObjectPath B){ return A.LexicalLess(B); });

		// Static collection. Save a flat list of all objects in the collection.
		for (const FSoftObjectPath& ObjectName : ObjectList)
		{
			FileOutput += ObjectName.ToString() + LINE_TERMINATOR;
		}
	}
	else
	{
		// Dynamic collection, a single query line
		FileOutput += DynamicQueryText + LINE_TERMINATOR;
	}

	// Attempt to save the file
	bool bSaveSuccessful = false;
	if ( ensure(FileOutput.Len()) )
	{
		// We have some output, write it to file
		if ( FFileHelper::SaveStringToFile(FileOutput, *SourceFilename) )
		{
			bSaveSuccessful = true;
		}
		else
		{
			OutError = FText::Format(LOCTEXT("Error_WriteFailed", "Failed to write to collection file: {0}"), FText::FromString(SourceFilename));
			UE_LOGF(LogCollectionManager, Error, "%ls", *OutError.ToString());
		}
	}
	else
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
	}

	GWarn->UpdateProgress(SaveProgressNumerator++, SaveProgressDenominator);

	if ( bSaveSuccessful )
	{
		if ( bUseSCC && (bForceCommitToRevisionControl || GetDefault<UCollectionSettings>()->bAutoCommitOnSave))
		{
			// Check in the file if the save was successful
			if ( bSaveSuccessful )
			{
				if ( !CheckinCollection(AdditionalChangelistText, OutError) )
				{
					UE_LOGF(LogCollectionManager, Error, "Failed to check in a collection successfully saving: %ls", *CollectionName.ToString());
					bSaveSuccessful = false;
				}
			}
		
			// If the save was not successful or the checkin failed, revert
			if ( !bSaveSuccessful )
			{
				FText Unused;
				if ( !RevertCollection(Unused) )
				{
					// The revert failed... file will be left on disk as it was saved.
					// DiskAssetList will still hold the version of the file when this collection was last loaded or saved successfully so nothing will be out of sync.
					// If the user closes the editor before successfully saving, this file may not be exactly what was seen at the time the editor closed.
					UE_LOGF(LogCollectionManager, Warning, "Failed to revert a checked out collection after failing to save or checkin: %ls", *CollectionName.ToString());
				}
			}
		}
	}

	GWarn->UpdateProgress(SaveProgressNumerator++, SaveProgressDenominator);

	if ( bSaveSuccessful )
	{
		// Files are always saved at the latest version as loading should take care of data upgrades
		FileVersion = ECollectionVersion::CurrentVersion;

		DiskSnapshot.TakeSnapshot(*this);
		bChangedSinceLastDiskSnapshot = false;
	}

	GWarn->EndSlowTask();

	UE_LOGF(LogCollectionManager, Verbose, "Saved collection %ls in %0.6f seconds", *CollectionName.ToString(), FPlatformTime::Seconds() - SaveStartTime);

	return bSaveSuccessful;
}

bool FCollection::Update(FText& OutError)
{
	if ( !ensure(SourceFilename.Len()) )
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		return false;
	}

	if ( !bUseSCC )
	{
		// Not under SCC control, so already up-to-date
		return true;
	}

	FScopedSlowTask SlowTask(1.0f, FText::Format(LOCTEXT("UpdatingCollection", "Updating Collection {0}"), FText::FromName(CollectionName )));

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutError = LOCTEXT("Error_SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	if ( !SourceControlProvider.IsAvailable() )
	{
		OutError = LOCTEXT("Error_SCCNotAvailable", "Revision control is currently not available. Check your connection and try again.");
		return false;
	}

	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

	// If not at the head revision, sync up
	if (SourceControlState.IsValid() && !SourceControlState->IsCurrent())
	{
		if ( SourceControlProvider.Execute(ISourceControlOperation::Create<FSync>(), AbsoluteFilename) == ECommandResult::Failed )
		{
			// Could not sync up with the head revision
			OutError = FText::Format(LOCTEXT("Error_SCCSync", "Failed to sync collection '{0}' to the head revision."), FText::FromName(CollectionName));
			return false;
		}

		// Check to see if the file exists at the head revision
		if ( IFileManager::Get().FileExists(*SourceFilename) )
		{
			// File found! Load it and merge with our local changes
			FText LoadErrorText;
			FCollection NewCollection(SourceFilename, false, ECollectionStorageMode::Static);
			if ( !NewCollection.Load(LoadErrorText) )
			{
				// Failed to load the head revision file so it isn't safe to delete it
				OutError = FText::Format(LOCTEXT("Error_SCCBadHead", "Failed to load the collection '{0}' at the head revision. {1}"), FText::FromName(CollectionName), LoadErrorText);
				return false;
			}

			// Loaded the head revision, now merge up so the files are in a consistent state
			MergeWithCollection(NewCollection);
		}

		// Make sure we get a fresh state from the server
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

		// Got an updated version?
		if (SourceControlState.IsValid() && !SourceControlState->IsCurrent())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCNotCurrent", "Collection '{0}' is not at head revision after sync."), FText::FromName(CollectionName));
			return false;
		}
	}

	return true;
}

bool FCollection::Merge(const FCollection& NewCollection)
{
	return MergeWithCollection(NewCollection);
}

bool FCollection::DeleteSourceFile(FText& OutError)
{
	bool bSuccessfullyDeleted = false;

	if ( SourceFilename.Len() )
	{
		if ( bUseSCC )
		{
			bSuccessfullyDeleted = DeleteFromSourceControl(OutError);
		}
		else
		{
			bSuccessfullyDeleted = IFileManager::Get().Delete(*SourceFilename);
			if ( !bSuccessfullyDeleted )
			{
				OutError = FText::Format(LOCTEXT("Error_DiskDeleteFailed", "Failed to delete the collection file: {0}"), FText::FromString(SourceFilename));
			}
		}
	}
	else
	{
		// No source file. Since it doesn't exist we will say it is deleted.
		bSuccessfullyDeleted = true;
	}

	if ( bSuccessfullyDeleted )
	{
		DiskSnapshot = FCollectionSnapshot();
		bChangedSinceLastDiskSnapshot = (ObjectSet.Num() == 0);
	}

	return bSuccessfullyDeleted;
}

void FCollection::Empty()
{
	ObjectSet.Reset();
	DynamicQueryText.Reset();
	DynamicQueryExpressionEvaluatorPtr.Reset();

	DiskSnapshot.TakeSnapshot(*this);
	bChangedSinceLastDiskSnapshot = false;
}

bool FCollection::AddObjectToCollection(const FSoftObjectPath& ObjectPath)
{
	if (ObjectPath.IsNull())
	{
		return false;
	}

	if (StorageMode == ECollectionStorageMode::Static)
	{
		bool bAlreadyInSet = false;
		ObjectSet.Add(ObjectPath, &bAlreadyInSet);
		bChangedSinceLastDiskSnapshot |= !bAlreadyInSet;
		return !bAlreadyInSet;
	}

	return false;
}

bool FCollection::RemoveObjectFromCollection(const FSoftObjectPath& ObjectPath)
{
	if (ObjectPath.IsNull())
	{
		return false;
	}

	if (StorageMode == ECollectionStorageMode::Static && ObjectSet.Remove(ObjectPath) > 0)
	{
		bChangedSinceLastDiskSnapshot = true;
		return true;
	}

	return false;
}

void FCollection::GetAssetsInCollection(TArray<FSoftObjectPath>& Assets) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		for (const FSoftObjectPath& ObjectName : ObjectSet)
		{
			if (!ObjectName.GetLongPackageName().StartsWith(TEXT("/Script/")))
			{
				Assets.Add(ObjectName);
			}
		}
	}
}

void FCollection::GetClassesInCollection(TArray<FTopLevelAssetPath>& Classes) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		for (const FSoftObjectPath& ObjectName : ObjectSet)
		{
			if (ObjectName.GetLongPackageName().StartsWith(TEXT("/Script/")))
			{
				Classes.Add(ObjectName.GetAssetPath());
			}
		}
	}
}

void FCollection::GetObjectsInCollection(TArray<FSoftObjectPath>& Objects) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		FCollectionUtils::AppendCollectionToArray(ObjectSet, Objects);
	}
}

bool FCollection::IsObjectInCollection(const FSoftObjectPath& ObjectPath) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		return ObjectSet.Contains(ObjectPath);
	}

	return false;
}

bool FCollection::IsRedirectorInCollection(const FSoftObjectPath& ObjectPath) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		// Redirectors are fixed up in-memory once the asset registry has finished loading, 
		// so we need to test our on-disk set of objects rather than our in-memory set of objects
		return DiskSnapshot.ObjectSet.Contains(ObjectPath);
	}

	return false;
}

bool FCollection::SetDynamicQueryText(const FString& InQueryText)
{
	if (StorageMode == ECollectionStorageMode::Dynamic)
	{
		DynamicQueryText = InQueryText;
		return true;
	}

	return false;
}

FString FCollection::GetDynamicQueryText() const
{
	return (StorageMode == ECollectionStorageMode::Dynamic) ? DynamicQueryText : FString();
}

void FCollection::PrepareDynamicQuery()
{
	if (StorageMode == ECollectionStorageMode::Dynamic)
	{
		UE::TUniqueLock Lock(*DynamicQueryExpressionEvaluatorLock);
		if (!DynamicQueryExpressionEvaluatorPtr.IsValid())
		{
			DynamicQueryExpressionEvaluatorPtr = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex));
		}

		if (!DynamicQueryExpressionEvaluatorPtr->GetFilterText().ToString().Equals(DynamicQueryText, ESearchCase::CaseSensitive))
		{
			DynamicQueryExpressionEvaluatorPtr->SetFilterText(FText::FromString(DynamicQueryText));
		}
	}
}

bool FCollection::TestDynamicQuery(const ITextFilterExpressionContext& InContext) const
{
	if (StorageMode == ECollectionStorageMode::Dynamic)
	{
		return DynamicQueryExpressionEvaluatorPtr->TestTextFilter(InContext);
	}

	return false;
}

FCollectionStatusInfo FCollection::GetStatusInfo(bool bIsReadOnly) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCollection::GetStatusInfo);

	FCollectionStatusInfo StatusInfo;

	StatusInfo.bIsDirty = !bIsReadOnly && IsDirty();
	StatusInfo.bIsEmpty = IsEmpty();
	StatusInfo.bUseSCC  = !bIsReadOnly && bUseSCC;

	StatusInfo.NumObjects = ObjectSet.Num();

	if (StatusInfo.bUseSCC && ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if (SourceControlProvider.IsAvailable())
		{
			const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
			StatusInfo.SCCState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::Use);
		}
	}

	return StatusInfo;
}

bool FCollection::IsDirty() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCollection::IsDirty);

	if (ParentCollectionGuid != DiskSnapshot.ParentCollectionGuid)
	{
		return true;
	}
	
	if (CollectionColor != DiskSnapshot.CollectionColor)
	{
		return true;
	}

	if (StorageMode == ECollectionStorageMode::Static)
	{
		return bChangedSinceLastDiskSnapshot;
	}
	else
	{
		return DynamicQueryText != DiskSnapshot.DynamicQueryText;
	}
}

bool FCollection::IsEmpty() const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		return ObjectSet.Num() == 0;
	}
	else
	{
		return DynamicQueryText.IsEmpty();
	}
}

void FCollection::PrintCollection() const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		UE_LOGF(LogCollectionManager, Log, "    Printing static elements of collection %ls", *CollectionName.ToString());
		UE_LOGF(LogCollectionManager, Log, "    =============================");

		// Print the set as a sorted array to keep things in a sane order
		TArray<FSoftObjectPath> ObjectList = ObjectSet.Array();
		ObjectList.Sort([](const FSoftObjectPath& A, const FSoftObjectPath& B){ return A.LexicalLess(B); });

		for (const FSoftObjectPath& ObjectName : ObjectList)
		{
			UE_LOGF(LogCollectionManager, Log, "        %ls", *ObjectName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogCollectionManager, Log, "    Printing dynamic query of collection %ls", *CollectionName.ToString());
		UE_LOGF(LogCollectionManager, Log, "    =============================");
		UE_LOGF(LogCollectionManager, Log, "        %ls", *DynamicQueryText);
	}
}

void FCollection::SaveHeaderPairs(TMap<FString,FString>& OutHeaderPairs) const
{
	// These pairs will appear at the top of the file followed by a newline
	OutHeaderPairs.Add(TEXT("FileVersion"), FString::FromInt(ECollectionVersion::CurrentVersion)); // Files are always saved at the latest version as loading should take care of data upgrades
	OutHeaderPairs.Add(TEXT("Type"), ECollectionStorageMode::ToString(StorageMode));
	OutHeaderPairs.Add(TEXT("Guid"), CollectionGuid.ToString(EGuidFormats::DigitsWithHyphens));
	OutHeaderPairs.Add(TEXT("ParentGuid"), ParentCollectionGuid.ToString(EGuidFormats::DigitsWithHyphens));
	OutHeaderPairs.Add(TEXT("IsAutoGenerated"), bIsAutoGenerated ? TEXT("true") : TEXT("false"));
	if (CollectionColor)
	{
		OutHeaderPairs.Add(TEXT("Color"), CollectionColor->ToString());
	}
}

bool FCollection::LoadHeaderPairs(const TMap<FString,FString>& InHeaderPairs)
{
	// These pairs will appeared at the top of the file being loaded
	// First find all the known pairs
	const FString* Version = InHeaderPairs.Find(TEXT("FileVersion"));
	if ( !Version )
	{
		// FileVersion is required
		return false;
	}

	const FString* Type = InHeaderPairs.Find(TEXT("Type"));
	if ( !Type )
	{
		// Type is required
		return false;
	}

	StorageMode = ECollectionStorageMode::FromString(**Type);

	FileVersion = (ECollectionVersion::Type)FCString::Atoi(**Version);

	if (FileVersion >= ECollectionVersion::AddedCollectionGuid)
	{
		const FString* GuidStr = InHeaderPairs.Find(TEXT("Guid"));
		if ( !GuidStr || !FGuid::Parse(*GuidStr, CollectionGuid) )
		{
			// Guid is required
			return false;
		}

		const FString* ParentGuidStr = InHeaderPairs.Find(TEXT("ParentGuid"));
		if ( !ParentGuidStr || !FGuid::Parse(*ParentGuidStr, ParentCollectionGuid) )
		{
			ParentCollectionGuid = FGuid();
		}
	}

	// Load the optional color
	CollectionColor.Reset();
	if (const FString* ColorStr = InHeaderPairs.Find(TEXT("Color")))
	{
		FLinearColor NewColor;
		if (NewColor.InitFromString(*ColorStr))
		{
			CollectionColor = MoveTemp(NewColor);
		}
	}

	bIsAutoGenerated = false;
	if (const FString* IsAutoGeneratedString = InHeaderPairs.Find(TEXT("IsAutoGenerated")))
	{
		bIsAutoGenerated = IsAutoGeneratedString->ToBool();
	}

	return FileVersion > 0 && FileVersion <= ECollectionVersion::CurrentVersion;
}

bool FCollection::MergeWithCollection(const FCollection& Other)
{
	bool bHasChanges = ParentCollectionGuid != Other.ParentCollectionGuid;

	ParentCollectionGuid = Other.ParentCollectionGuid;

	if (StorageMode != Other.StorageMode)
	{
		bHasChanges = true;
		StorageMode = Other.StorageMode;

		// Storage mode has changed! Empty the collection so we just copy over the new data verbatim
		Empty();
	}

	if (StorageMode == ECollectionStorageMode::Static)
	{
		// Work out whether we have any changes compared to the other collection
		TArray<FSoftObjectPath> ObjectsAdded;
		TArray<FSoftObjectPath> ObjectsRemoved;
		GetObjectDifferences(ObjectSet, Other.ObjectSet, ObjectsAdded, ObjectsRemoved);

		bHasChanges = bHasChanges || ObjectsAdded.Num() > 0 || ObjectsRemoved.Num() > 0;

		if (bHasChanges)
		{
			// Gather the differences from the file on disk
			ObjectsAdded.Reset();
			ObjectsRemoved.Reset();
			GetObjectDifferencesFromDisk(ObjectsAdded, ObjectsRemoved);

			// Copy asset list from other collection
			ObjectSet = Other.ObjectSet;

			// Add the objects that were added before the merge
			for (const FSoftObjectPath& AddedObjectName : ObjectsAdded)
			{
				ObjectSet.Add(AddedObjectName);
			}

			// Remove the objects that were removed before the merge
			for (const FSoftObjectPath& RemovedObjectName : ObjectsRemoved)
			{
				ObjectSet.Remove(RemovedObjectName);
			}
			
			bChangedSinceLastDiskSnapshot = true;
		}
	}
	else
	{
		bHasChanges = bHasChanges || DynamicQueryText != Other.DynamicQueryText;
		DynamicQueryText = Other.DynamicQueryText;
	}

	DiskSnapshot = Other.DiskSnapshot;

	return bHasChanges;
}

void FCollection::GetObjectDifferences(const TSet<FSoftObjectPath>& BaseSet, const TSet<FSoftObjectPath>& NewSet, TArray<FSoftObjectPath>& ObjectsAdded, TArray<FSoftObjectPath>& ObjectsRemoved)
{
	// Find the objects that were removed compared to the base set
	for (const FSoftObjectPath& BaseObjectName : BaseSet)
	{
		if (!NewSet.Contains(BaseObjectName))
		{
			ObjectsRemoved.Add(BaseObjectName);
		}
	}

	// If both sets have the same number of items and nothing has been removed
	// we can safely infer that both collections are equals without going
	// over them a second time.
	if (ObjectsRemoved.Num() == 0 && BaseSet.Num() == NewSet.Num())
	{
		return;
	}

	// Find the objects that were added compare to the base set
	for (const FSoftObjectPath& NewObjectName : NewSet)
	{
		if (!BaseSet.Contains(NewObjectName))
		{
			ObjectsAdded.Add(NewObjectName);
		}
	}
}

void FCollection::GetObjectDifferencesFromDisk(TArray<FSoftObjectPath>& ObjectsAdded, TArray<FSoftObjectPath>& ObjectsRemoved) const
{
	if (StorageMode == ECollectionStorageMode::Static)
	{
		GetObjectDifferences(DiskSnapshot.ObjectSet, ObjectSet, ObjectsAdded, ObjectsRemoved);
	}
}

bool FCollection::CheckoutCollection(FText& OutError)
{
	if ( !ensure(SourceFilename.Len()) )
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		return false;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutError = LOCTEXT("Error_SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	if ( !SourceControlProvider.IsAvailable() )
	{
		OutError = LOCTEXT("Error_SCCNotAvailable", "Revision control is currently not available. Check your connection and try again.");
		return false;
	}

	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

	bool bSuccessfullyCheckedOut = false;

	if (SourceControlState.IsValid() && SourceControlState->IsDeleted())
	{
		// Revert our delete
		if ( !RevertCollection(OutError) )
		{
			return false;
		}

		// Make sure we get a fresh state from the server
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	}

	// If not at the head revision, sync up
	if (SourceControlState.IsValid() && !SourceControlState->IsCurrent())
	{
		if ( SourceControlProvider.Execute(ISourceControlOperation::Create<FSync>(), AbsoluteFilename) == ECommandResult::Failed )
		{
			// Could not sync up with the head revision
			OutError = FText::Format(LOCTEXT("Error_SCCSync", "Failed to sync collection '{0}' to the head revision."), FText::FromName(CollectionName));
			return false;
		}

		// Check to see if the file exists at the head revision
		if ( IFileManager::Get().FileExists(*SourceFilename) )
		{
			// File found! Load it and merge with our local changes
			FText LoadErrorText;
			FCollection NewCollection(SourceFilename, false, ECollectionStorageMode::Static);
			if ( !NewCollection.Load(LoadErrorText) )
			{
				// Failed to load the head revision file so it isn't safe to delete it
				OutError = FText::Format(LOCTEXT("Error_SCCBadHead", "Failed to load the collection '{0}' at the head revision. {1}"), FText::FromName(CollectionName), LoadErrorText);
				return false;
			}

			// Loaded the head revision, now merge up so the files are in a consistent state
			MergeWithCollection(NewCollection);
		}

		// Make sure we get a fresh state from the server
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	}

	if(SourceControlState.IsValid())
	{
		if(!SourceControlState->IsSourceControlled())
		{
			// Not yet in the depot. We'll add it when we call CheckinCollection
			bSuccessfullyCheckedOut = true;
		}
		else if(SourceControlState->IsAdded() || SourceControlState->IsCheckedOut())
		{
			// Already checked out or opened for add
			bSuccessfullyCheckedOut = true;
		}
		else if(SourceControlState->CanCheckout())
		{
			// In depot and needs to be checked out
			bSuccessfullyCheckedOut = (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), AbsoluteFilename) == ECommandResult::Succeeded);
			if (!bSuccessfullyCheckedOut)
			{
				OutError = FText::Format(LOCTEXT("Error_SCCCheckout", "Failed to check out collection '{0}'"), FText::FromName(CollectionName));
			}
		}
		else if(!SourceControlState->IsCurrent())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCNotCurrent", "Collection '{0}' is not at head revision after sync."), FText::FromName(CollectionName));
		}
		else if(SourceControlState->IsCheckedOutOther())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCCheckedOutOther", "Collection '{0}' is checked out by another user."), FText::FromName(CollectionName));
		}
		else
		{
			OutError = FText::Format(LOCTEXT("Error_SCCUnknown", "Could not determine revision control state for collection '{0}'"), FText::FromName(CollectionName));
		}
	}
	else
	{
		OutError = LOCTEXT("Error_SCCInvalid", "Revision control state is invalid.");
	}

	return bSuccessfullyCheckedOut;
}

bool FCollection::CheckinCollection(const TArray<FText>& AdditionalChangelistText, FText& OutError)
{
	if ( !ensure(SourceFilename.Len()) )
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		return false;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutError = LOCTEXT("Error_SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	if ( !SourceControlProvider.IsAvailable() )
	{
		OutError = LOCTEXT("Error_SCCNotAvailable", "Revision control is currently not available. Check your connection and try again.");
		return false;
	}

	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

	if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
	{
		// Not yet in the depot. Add it.
		const bool bWasAdded = (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), AbsoluteFilename) == ECommandResult::Succeeded);
		if (!bWasAdded)
		{
			OutError = FText::Format(LOCTEXT("Error_SCCAdd", "Failed to add collection '{0}' to revision control."), FText::FromName(CollectionName));
			return false;
		}
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	}

	if ( SourceControlState.IsValid() && !(SourceControlState->IsCheckedOut() || SourceControlState->IsAdded()) )
	{
		OutError = FText::Format(LOCTEXT("Error_SCCNotCheckedOut", "Collection '{0}' not checked out or open for add."), FText::FromName(CollectionName));
		return false;
	}

	// Form an appropriate summary for the changelist
	const FText CollectionNameText = FText::FromName( CollectionName );
	FTextBuilder ChangelistDescBuilder;

	if (SourceControlState.IsValid() && SourceControlState->IsAdded())
	{
		ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionAddedNewDesc", "Added collection '{0}'"), CollectionNameText);
	}
	else
	{
		if (StorageMode == ECollectionStorageMode::Static)
		{
			auto AddFileListToDescription = [&ChangelistDescBuilder](const TArray<FSoftObjectPath>& Paths)
			{
				const int32 MaxPaths = CVarCollectionsMaxCLDescriptionPathCount.GetValueOnAnyThread();
				const int32 ReportedPaths = FMath::Min(Paths.Num(), MaxPaths);
				const int32 UnreportedPaths = FMath::Max(0, Paths.Num() - MaxPaths);
				for (int32 PathIdx = 0; PathIdx < ReportedPaths; ++PathIdx)
				{
					const FSoftObjectPath& AddedObjectName = Paths[PathIdx];
					ChangelistDescBuilder.AppendLine(FText::FromString(AddedObjectName.ToString()));
				}
				if (UnreportedPaths > 0)
				{
					ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionUnreportedPathsDesc", "... {0} more path(s)"), UnreportedPaths);
				}
			};

			// Gather differences from disk
			TArray<FSoftObjectPath> ObjectsAdded;
			TArray<FSoftObjectPath> ObjectsRemoved;
			GetObjectDifferencesFromDisk(ObjectsAdded, ObjectsRemoved);

			ObjectsAdded.Sort([](FSoftObjectPath A, FSoftObjectPath B){ return A.LexicalLess(B); });
			ObjectsRemoved.Sort([](FSoftObjectPath A, FSoftObjectPath B) { return A.LexicalLess(B); });

			// Report added files
			FFormatNamedArguments Args;
			Args.Add(TEXT("FirstObjectAdded"), ObjectsAdded.Num() > 0 ? FText::FromString(ObjectsAdded[0].ToString()) : NSLOCTEXT("Core", "None", "None"));
			Args.Add(TEXT("NumberAdded"), FText::AsNumber(ObjectsAdded.Num()));
			Args.Add(TEXT("FirstObjectRemoved"), ObjectsRemoved.Num() > 0 ? FText::FromString(ObjectsRemoved[0].ToString()) : NSLOCTEXT("Core", "None", "None"));
			Args.Add(TEXT("NumberRemoved"), FText::AsNumber(ObjectsRemoved.Num()));
			Args.Add(TEXT("CollectionName"), CollectionNameText);

			if (ObjectsAdded.Num() == 1)
			{
				ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionAddedSingleDesc", "Added '{FirstObjectAdded}' to collection '{CollectionName}'"), Args);
			}
			else if (ObjectsAdded.Num() > 1)
			{
				ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionAddedMultipleDesc", "Added {NumberAdded} objects to collection '{CollectionName}':"), Args);

				ChangelistDescBuilder.Indent();
				AddFileListToDescription(ObjectsAdded);
				ChangelistDescBuilder.Unindent();
			}

			if ( ObjectsRemoved.Num() == 1 )
			{
				ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionRemovedSingleDesc", "Removed '{FirstObjectRemoved}' from collection '{CollectionName}'"), Args);
			}
			else if (ObjectsRemoved.Num() > 1)
			{
				ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionRemovedMultipleDesc", "Removed {NumberRemoved} objects from collection '{CollectionName}'"), Args);

				ChangelistDescBuilder.Indent();
				AddFileListToDescription(ObjectsRemoved);
				ChangelistDescBuilder.Unindent();
			}
		}
		else
		{
			if (DiskSnapshot.DynamicQueryText != DynamicQueryText)
			{
				ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionChangedDynamicQueryDesc", "Changed the dynamic query of collection '{0}' to '{1}'"), CollectionNameText, FText::FromString(DynamicQueryText));
			}
		}

		// Parent change?
		if (DiskSnapshot.ParentCollectionGuid != ParentCollectionGuid)
		{
			ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionChangedParentDesc", "Changed the parent of collection '{0}'"), CollectionNameText);
		}

		// Color change?
		if (DiskSnapshot.CollectionColor != CollectionColor)
		{
			ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionChangedColorDesc", "Changed the color of collection '{0}'"), CollectionNameText);
		}

		// Version bump?
		if (FileVersion < ECollectionVersion::CurrentVersion)
		{
			ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionUpgradedDesc", "Upgraded collection '{0}' (was version {1}, now version {2})"), CollectionNameText, FText::AsNumber(FileVersion), FText::AsNumber(ECollectionVersion::CurrentVersion));
		}
	}

	if (ChangelistDescBuilder.IsEmpty())
	{
		// No changes could be detected
		ChangelistDescBuilder.AppendLineFormat(LOCTEXT("CollectionNotModifiedDesc", "Collection '{0}' not modified"), CollectionNameText);
	}

	for (const FText& AdditionalText : AdditionalChangelistText)
	{
		ChangelistDescBuilder.AppendLine(AdditionalText);
	}

	FText ChangelistDesc = ChangelistDescBuilder.ToText();

	// Finally check in the file
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
	CheckInOperation->SetDescription( ChangelistDesc );
	if ( SourceControlProvider.Execute( CheckInOperation, AbsoluteFilename ) )
	{
		return true;
	}
	else 
	{
		OutError = FText::Format(LOCTEXT("Error_SCCCheckIn", "Failed to check in collection '{0}'."), FText::FromName(CollectionName));
		return false;
	}
}

bool FCollection::RevertCollection(FText& OutError)
{
	if ( !ensure(SourceFilename.Len()) )
	{
		OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		return false;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutError = LOCTEXT("Error_SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	if ( !SourceControlProvider.IsAvailable() )
	{
		OutError = LOCTEXT("Error_SCCNotAvailable", "Revision control is currently not available. Check your connection and try again.");
		return false;
	}

	FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

	if ( SourceControlState.IsValid() && !(SourceControlState->IsCheckedOut() || SourceControlState->IsAdded()) )
	{
		OutError = FText::Format(LOCTEXT("Error_SCCNotCheckedOut", "Collection '{0}' not checked out or open for add."), FText::FromName(CollectionName));
		return false;
	}

	if ( SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), AbsoluteFilename) == ECommandResult::Succeeded)
	{
		return true;
	}
	else
	{
		OutError = FText::Format(LOCTEXT("Error_SCCRevert", "Could not revert collection '{0}'"), FText::FromName(CollectionName));
		return false;
	}
}

bool FCollection::DeleteFromSourceControl(FText& OutError)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutError = LOCTEXT("Error_SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	if ( !SourceControlProvider.IsAvailable() )
	{
		OutError = LOCTEXT("Error_SCCNotAvailable", "Revision control is currently not available. Check your connection and try again.");
		return false;
	}

	bool bDeletedSuccessfully = false;

	const int32 DeleteProgressDenominator = 2;
	int32 DeleteProgressNumerator = 0;

	const FText CollectionNameText = FText::FromName( CollectionName );

	FFormatNamedArguments Args;
	Args.Add( TEXT("CollectionName"), CollectionNameText );
	const FText StatusUpdate = FText::Format( LOCTEXT("DeletingCollection", "Deleting Collection {CollectionName}"), Args );

	GWarn->BeginSlowTask( StatusUpdate, true );
	GWarn->UpdateProgress(DeleteProgressNumerator++, DeleteProgressDenominator);

	FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SourceFilename);
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	
	GWarn->UpdateProgress(DeleteProgressNumerator++, DeleteProgressDenominator);

	// If checked out locally for some reason, revert
	if (SourceControlState.IsValid() && (SourceControlState->IsAdded() || SourceControlState->IsCheckedOut() || SourceControlState->IsDeleted()))
	{
		if ( !RevertCollection(OutError) )
		{
			// Failed to revert, just bail out
			GWarn->EndSlowTask();
			return false;
		}

		// Make sure we get a fresh state from the server
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	}

	// If not at the head revision, sync up
	if (SourceControlState.IsValid() && !SourceControlState->IsCurrent())
	{
		if ( SourceControlProvider.Execute(ISourceControlOperation::Create<FSync>(), AbsoluteFilename) == ECommandResult::Failed)
		{
			// Could not sync up with the head revision
			GWarn->EndSlowTask();
			OutError = FText::Format(LOCTEXT("Error_SCCSync", "Failed to sync collection '{0}' to the head revision."), FText::FromName(CollectionName));
			return false;
		}

		// Check to see if the file exists at the head revision
		if ( !IFileManager::Get().FileExists(*SourceFilename) )
		{
			// File was already deleted, consider this a success
			GWarn->EndSlowTask();
			return true;
		}
			
		FCollection NewCollection(SourceFilename, false, ECollectionStorageMode::Static);
		FText LoadErrorText;
		if ( !NewCollection.Load(LoadErrorText) )
		{
			// Failed to load the head revision file so it isn't safe to delete it
			GWarn->EndSlowTask();
			OutError = FText::Format(LOCTEXT("Error_SCCBadHead", "Failed to load the collection '{0}' at the head revision. {1}"), FText::FromName(CollectionName), LoadErrorText);
			return false;
		}

		// Loaded the head revision, now merge up so the files are in a consistent state
		MergeWithCollection(NewCollection);

		// Make sure we get a fresh state from the server
		SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	}

	GWarn->UpdateProgress(DeleteProgressNumerator++, DeleteProgressDenominator);

	if(SourceControlState.IsValid())
	{
		if(SourceControlState->IsAdded() || SourceControlState->IsCheckedOut())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCDeleteWhileCheckedOut", "Failed to delete collection '{0}' in revision control because it is checked out or open for add."), FText::FromName(CollectionName));
		}
		else if(SourceControlState->CanCheckout())
		{
			if ( SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), AbsoluteFilename) == ECommandResult::Succeeded )
			{
				// Now check in the delete
				const FText ChangelistDesc = FText::Format( LOCTEXT("CollectionDeletedDesc", "Deleted collection: {CollectionName}"), Args );
				TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
				CheckInOperation->SetDescription(ChangelistDesc);
				if ( SourceControlProvider.Execute( CheckInOperation, AbsoluteFilename ) )
				{
					// Deleted successfully!
					bDeletedSuccessfully = true;
				}
				else
				{
					FText Unused;
					if ( !RevertCollection(Unused) )
					{
						UE_LOGF(LogCollectionManager, Warning, "Failed to revert collection '%ls' after failing to check in the file that was marked for delete.", *CollectionName.ToString());
					}

					OutError = FText::Format(LOCTEXT("Error_SCCCheckIn", "Failed to check in collection '{0}'."), FText::FromName(CollectionName));
				}
			}
			else
			{
				OutError = FText::Format(LOCTEXT("Error_SCCDeleteFailed", "Failed to delete collection '{0}' in revision control."), FText::FromName(CollectionName));
			}
		}
		else if(!SourceControlState->IsSourceControlled())
		{
			// Not yet in the depot or deleted. We can just delete it from disk.
			bDeletedSuccessfully = IFileManager::Get().Delete(*AbsoluteFilename);
			if ( !bDeletedSuccessfully )
			{
				OutError = FText::Format(LOCTEXT("Error_DiskDeleteFailed", "Failed to delete the collection file: {0}"), FText::FromString(AbsoluteFilename));
			}
		}
		else if (!SourceControlState->IsCurrent())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCNotCurrent", "Collection '{0}' is not at head revision after sync."), FText::FromName(CollectionName));
		}
		else if(SourceControlState->IsCheckedOutOther())
		{
			OutError = FText::Format(LOCTEXT("Error_SCCCheckedOutOther", "Collection '{0}' is checked out by another user."), FText::FromName(CollectionName));
		}
		else
		{
			OutError = FText::Format(LOCTEXT("Error_SCCUnknown", "Could not determine revision control state for collection '{0}'"), FText::FromName(CollectionName));
		}
	}
	else
	{
		OutError = LOCTEXT("Error_SCCInvalid", "Revision control state is invalid.");
	}

	GWarn->UpdateProgress(DeleteProgressNumerator++, DeleteProgressDenominator);

	GWarn->EndSlowTask();

	return bDeletedSuccessfully;
}


#undef LOCTEXT_NAMESPACE
