// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "CollectionManagerTypes.h"
#include "Templates/DontCopy.h"
#include "Async/Mutex.h"

class FTextFilterExpressionEvaluator;
class ITextFilterExpressionContext;

struct ECollectionVersion
{
	enum Type
	{
		_ZeroVersion = 0,

		/** The initial version for collection files */
		Initial,

		/** 
		 * Added GUIDs to collections to allow them to be used as a parent of another collection without relying on their name/share-type combo
		 * Collections that are older than this version must be re-saved before they can be used as a parent for another collection
		 */
		AddedCollectionGuid,

		/** -----<new versions can be added before this line>------------------------------------------------- */
		_AutomaticVersionPlusOne,
		CurrentVersion = _AutomaticVersionPlusOne - 1
	};
};

enum class ECollectionCloneMode : uint8
{
	/** Clone this collection exactly as it is now, preserving its existing GUID data */
	Exact,
	/** Clone this collection, but make sure it gets unique GUIDs */
	Unique,
};

/**
 * A class to represent a collection of assets.
 * Collections are held privately by the collection manager and most thread safety is dealt with by locking at the collection manager level.
 */
class FCollection
{
public:
	FCollection(const FString& InFilename, bool InUseSCC, ECollectionStorageMode::Type InStorageMode);

	/** Clone this collection to a new location */
	TSharedRef<FCollection> Clone(const FString& InFilename, bool InUseSCC, ECollectionCloneMode InCloneMode) const;

	/** Loads content from the SourceFilename into this collection. If false, OutError is a human readable warning depicting the error. */
	bool Load(FText& OutError);
	/** Saves this collection to SourceFilename. If submitting to source control, AdditionalChangelistText will be added to the changelist description. If false, OutError is a human readable warning depicting the error. */
	bool Save(const TArray<FText>& AdditionalChangelistText, FText& OutError, bool bForceCommitToRevisionControl);
	/** Updates this collection to ensure it's the latest version from source control. If false, OutError is a human readable warning depicting the error. */
	bool Update(FText& OutError);
	/** Merge the contents of NewCollection into this collection. Returns true if there were changes to merge, or false if the collections were identical. */
	bool Merge(const FCollection& NewCollection);
	/** Deletes the source file for this collection. If false, OutError is a human readable warning depicting the error. */
	bool DeleteSourceFile(FText& OutError);
	/** Empty this collection */
	void Empty();

	/** Adds a single object to the collection. Static collections only. */
	bool AddObjectToCollection(const FSoftObjectPath& ObjectPath);
	/** Removes a single object from the collection. Static collections only. */
	bool RemoveObjectFromCollection(const FSoftObjectPath& ObjectPath);
	/** Gets a list of assets in the collection. Static collections only. */
	void GetAssetsInCollection(TArray<FSoftObjectPath>& Assets) const;
	/** Gets a list of native classes in the collection. Static collections only. */
	void GetClassesInCollection(TArray<FTopLevelAssetPath>& Classes) const;
	/** Gets a list of objects in the collection. Static collections only. */
	void GetObjectsInCollection(TArray<FSoftObjectPath>& Objects) const;
	/** Returns true when the specified object is in the collection. Static collections only. */
	bool IsObjectInCollection(const FSoftObjectPath& ObjectPath) const;
	/** Returns true when the specified redirector is in the collection. Static collections only. */
	bool IsRedirectorInCollection(const FSoftObjectPath& ObjectPath) const;

	/** Set the dynamic query text for this collection. Dynamic collections only. */
	bool SetDynamicQueryText(const FString& InQueryText);
	/** Get the dynamic query text for this collection. Dynamic collections only. */
	FString GetDynamicQueryText() const;
	/** 
	 * Prepares an expression evaluator for TestDynamicQuery if it has not yet been created or is out of date.
	 * The collection manager must prevent SetDynamicQueryText from being called between PrepareDynamicQuery and TestDynamicQuery.
	 * Internally locks against concurrent calls to PrepareDynamicQuery from under a shared read lock.
	 */
	void PrepareDynamicQuery();
	/** Tests the dynamic query for against the context provided. Dynamic collections only. Must call PrepareDynamicQuery first. */
	bool TestDynamicQuery(const ITextFilterExpressionContext& InContext) const;

	/** Get the status info for this collection */
	FCollectionStatusInfo GetStatusInfo(bool bIsReadOnly) const;

	/** Does this collection contain unsaved changes? */
	bool IsDirty() const;

	/** Whether the collection has any contents */
	bool IsEmpty() const;

	/** Whether the Collection uses Source Control */
	bool IsUsingSourceControl() const;

	/** Logs the contents of the collection */
	void PrintCollection() const;

	/** Returns the name of the collection */
	FORCEINLINE const FName& GetCollectionName() const { return CollectionName; }

	/** Returns the GUID of the collection */
	FORCEINLINE const FGuid& GetCollectionGuid() const { return CollectionGuid; }

	/** Returns the GUID of the collection we are parented under */
	FORCEINLINE const FGuid& GetParentCollectionGuid() const { return ParentCollectionGuid; }

	/** Set the GUID of the collection we are parented under */
	FORCEINLINE void SetParentCollectionGuid(const FGuid& NewGuid) { ParentCollectionGuid = NewGuid; }

	/** Get whether this collection is auto-generated by something external instead of maintained by users */
	FORCEINLINE bool IsAutoGenerated() const { return bIsAutoGenerated; }

	/** Get the color of the collection (if any) */
	FORCEINLINE TOptional<FLinearColor> GetCollectionColor() const { return CollectionColor; }

	/** Set the color of the collection (if any) */
	FORCEINLINE void SetCollectionColor(const TOptional<FLinearColor>& NewColor) { CollectionColor = NewColor; }

	/** Returns the file version of the collection */
	FORCEINLINE ECollectionVersion::Type GetCollectionVersion() const { return FileVersion; }

	/** Get whether this collection is static or dynamic */
	ECollectionStorageMode::Type GetStorageMode() const { return StorageMode; }

	/** Get the source filename of this collection */
	FORCEINLINE const FString& GetSourceFilename() const { return SourceFilename; }

	/** Gets set of all objects for fast iteration */
	FORCEINLINE const TSet<FSoftObjectPath>& GetObjectSet() const { return ObjectSet; }

private:
	/** Generates the header pairs for the collection file. */
	void SaveHeaderPairs(TMap<FString,FString>& OutHeaderPairs) const;

	/** 
	  * Processes header pairs from the top of a collection file.
	  *
	  * @param InHeaderPairs The header pairs found at the start of a collection file
	  * @return true if the header was valid and loaded properly
	  */
	bool LoadHeaderPairs(const TMap<FString,FString>& InHeaderPairs);

	template <typename CharType>
	bool ParseCollection(TStringView<CharType> FullFileContentsString, FText& OutError);

	/** Merges the assets from the specified collection with this collection */
	bool MergeWithCollection(const FCollection& Other);
	/** Gets the object differences between object set A (base) and B (new) */
	static void GetObjectDifferences(const TSet<FSoftObjectPath>& BaseSet, const TSet<FSoftObjectPath>& NewSet, TArray<FSoftObjectPath>& ObjectsAdded, TArray<FSoftObjectPath>& ObjectsRemoved);
	/** Gets the object differences between what we have in memory, and what we loaded from disk. Static collections only. */
	void GetObjectDifferencesFromDisk(TArray<FSoftObjectPath>& ObjectsAdded, TArray<FSoftObjectPath>& ObjectsRemoved) const;
	/** Checks the shared collection out from source control so it may be saved. If false, OutError is a human readable warning depicting the error. */
	bool CheckoutCollection(FText& OutError);
	/** Checks the shared collection in to source control after it is saved. In addition to the normal text, AdditionalChangelistText will be added to the checkin description. If false, OutError is a human readable warning depicting the error. */
	bool CheckinCollection(const TArray<FText>& AdditionalChangelistText, FText& OutError);
	/** Reverts the collection in the event that the save was not successful. If false, OutError is a human readable warning depicting the error.*/
	bool RevertCollection(FText& OutError);
	/** Marks the source file for delete in source control. If false, OutError is a human readable warning depicting the error. */
	bool DeleteFromSourceControl(FText& OutError);

private:
	/** Snapshot data for a collection. Used to take snapshots and provide a diff message */
	struct FCollectionSnapshot
	{
		void TakeSnapshot(const FCollection& InCollection)
		{
			ParentCollectionGuid = InCollection.ParentCollectionGuid;
			CollectionColor = InCollection.CollectionColor;
			ObjectSet = InCollection.ObjectSet;
			DynamicQueryText = InCollection.DynamicQueryText;
		}

		/** The GUID of the collection we are parented under */
		FGuid ParentCollectionGuid;

		/** The color of the collection (if set) */
		TOptional<FLinearColor> CollectionColor;

		/** The set of objects in the collection. Takes the form PackageName.AssetName. Static collections only. */
		TSet<FSoftObjectPath> ObjectSet; // TODO: could be FTopLevelAssetPath

		/** The dynamic query string for this collection. Dynamic collections only. */
		FString DynamicQueryText;
	};

	/** The name of the collection */
	FName CollectionName;

	/** The GUID of the collection */
	FGuid CollectionGuid;

	/** The GUID of the collection we are parented under */
	FGuid ParentCollectionGuid;

	/** The color of the collection (if any) */
	TOptional<FLinearColor> CollectionColor;

	/** Source control is used if true */
	bool bUseSCC;

	/** Indicates if the collection has changes in memory that weren't saved to disk. */
	bool bChangedSinceLastDiskSnapshot = false;

	/** Indicates if the collection is auto-generated by something external instead of maintained by users */
	bool bIsAutoGenerated = false;

	/** The filename used to load this collection. Empty if it is new or never loaded from disk. */
	FString SourceFilename;

	/** The set of objects in the collection. Takes the form PackageName.AssetName.Static collections only. */
	TSet<FSoftObjectPath> ObjectSet; // TODO: Could be FTopLevelAssetPath

	/** The dynamic query string for this collection. Dynamic collections only. */
	FString DynamicQueryText;

	/** Lock preventing concurrent instantiation of DynamicQueryExpressionEvaluatorPtr */
	TDontCopy<UE::FMutex> DynamicQueryExpressionEvaluatorLock;

	/** Expression evaluator that can be used test against the compiled DynamicQueryText */
	TSharedPtr<FTextFilterExpressionEvaluator> DynamicQueryExpressionEvaluatorPtr;

	/** The file version for this collection */
	ECollectionVersion::Type FileVersion;

	/** How does this collection store its objects? (static or dynamic) */
	ECollectionStorageMode::Type StorageMode;

	/** The state of the collection the last time it was loaded from or saved to disk. */
	FCollectionSnapshot DiskSnapshot;
};

inline bool FCollection::IsUsingSourceControl() const
{
	return bUseSCC;
}
