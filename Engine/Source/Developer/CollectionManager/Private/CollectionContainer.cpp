// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionContainer.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Containers/Ticker.h"
#include "CollectionManager.h"
#include "CollectionManagerLog.h"
#include "CollectionManagerModule.h"
#include "ICollectionSource.h"
#include "FileCache.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeRWLock.h"
#include "Async/ParallelFor.h"
#include "Misc/CommandLine.h"
#include "Tasks/Task.h"

#define LOCTEXT_NAMESPACE "CollectionManager"

FCollectionRecursiveRwLock::FThreadLockDepths::FThreadLockDepths(void* TlsSlotValue)
{
	ThreadReadDepth = static_cast<HalfUPtrInt>(reinterpret_cast<UPTRINT>(TlsSlotValue));
	ThreadWriteDepth = static_cast<HalfUPtrInt>(reinterpret_cast<UPTRINT>(TlsSlotValue) >> (sizeof(HalfUPtrInt) * 8));
}

void* FCollectionRecursiveRwLock::FThreadLockDepths::GetTlsSlotValue()
{
	UPTRINT ValueOut = static_cast<UPTRINT>(ThreadReadDepth) | (static_cast<UPTRINT>(ThreadWriteDepth) << (sizeof(HalfUPtrInt) * 8));
	return reinterpret_cast<void*>(ValueOut);
}

FCollectionRecursiveRwLock::FCollectionRecursiveRwLock()
{
	TlsSlot = FPlatformTLS::AllocTlsSlot();

	check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
}

FCollectionRecursiveRwLock::~FCollectionRecursiveRwLock()
{
	check(FPlatformTLS::IsValidTlsSlot(TlsSlot));

	FPlatformTLS::FreeTlsSlot(TlsSlot);
}

void FCollectionRecursiveRwLock::ReadLock()
{
	FThreadLockDepths LockDepths(FPlatformTLS::GetTlsValue(TlsSlot));
	LockDepths.ThreadReadDepth++;
	FPlatformTLS::SetTlsValue(TlsSlot, LockDepths.GetTlsSlotValue());

	if (LockDepths.ThreadReadDepth + LockDepths.ThreadWriteDepth == 1)
	{
		RwLock.ReadLock();
	}
}

void FCollectionRecursiveRwLock::WriteLock()
{
	// Some collection operations under write lock may cause Slate to tick if they show a dialog (e.g. FSlowTask),
	// which in turn may enter back into the collection container to retrieve data to update the UI.
	// Since this is difficult to avoid, we use a recursive read/write lock similar to the FPhysicsRwLock implementation.
	FThreadLockDepths LockDepths(FPlatformTLS::GetTlsValue(TlsSlot));
	LockDepths.ThreadWriteDepth++;
	FPlatformTLS::SetTlsValue(TlsSlot, LockDepths.GetTlsSlotValue());

	// If we have a read lock, we can't reuse it as a write lock without releasing it first.
	// Call PromoteInterruptible to do so.
	// If this situation arises because a thread with a read lock reenters and calls a high level function that requires
	// a write lock, then we need to reconsider this solution (we'll likely need to lift all the slow tasks and source
	// control operations out of the locks to prevent reentry altogether).
	checkf(LockDepths.ThreadReadDepth == 0, TEXT("Attempting to acquire a write lock on a collection container when the same thread already holds a read lock higher up the call stack."));

	if (LockDepths.ThreadReadDepth + LockDepths.ThreadWriteDepth == 1)
	{
		RwLock.WriteLock();
	}
}

void FCollectionRecursiveRwLock::ReadUnlock()
{
	FThreadLockDepths LockDepths(FPlatformTLS::GetTlsValue(TlsSlot));
	LockDepths.ThreadReadDepth--;
	FPlatformTLS::SetTlsValue(TlsSlot, LockDepths.GetTlsSlotValue());

	if (LockDepths.ThreadWriteDepth + LockDepths.ThreadReadDepth == 0)
	{
		RwLock.ReadUnlock();
	}
}

void FCollectionRecursiveRwLock::WriteUnlock()
{
	FThreadLockDepths LockDepths(FPlatformTLS::GetTlsValue(TlsSlot));
	LockDepths.ThreadWriteDepth--;
	FPlatformTLS::SetTlsValue(TlsSlot, LockDepths.GetTlsSlotValue());

	if (LockDepths.ThreadWriteDepth + LockDepths.ThreadReadDepth == 0)
	{
		RwLock.WriteUnlock();
	}
}

bool FCollectionRecursiveRwLock::PromoteInterruptible()
{
	FThreadLockDepths LockDepths(FPlatformTLS::GetTlsValue(TlsSlot));

	if (LockDepths.ThreadWriteDepth > 0 && LockDepths.ThreadReadDepth == 0)
	{
		// Already promoted.
		return true;
	}

	// Expecting this to be called from a thread with a read lock.
	checkf(LockDepths.ThreadReadDepth > 0, TEXT("Attempting to promote a read lock to a write lock on a collection container when the thread isn't holding a read lock."));

	// We cannot promote if we're not the topmost holder of the lock on this thread since we need to release the read lock and there might be another
	// thread trying to enter write lock.
	// 
	// Any code directly calling PromoteInterruptible is expecting to be pre-empted and will reconfirm its invariants and reacquire any pointers which may have been invalidated.
	// 
	// If there is another scope higher up on this thread holding a read lock, it is not expecting to be pre-empted and will not be prepared for its invariants to be broken or
	// its pointers to be invalidated.
	if (LockDepths.ThreadWriteDepth == 0 && LockDepths.ThreadReadDepth == 1)
	{
		LockDepths.ThreadWriteDepth = 1;
		LockDepths.ThreadReadDepth = 0;
		FPlatformTLS::SetTlsValue(TlsSlot, LockDepths.GetTlsSlotValue());

		RwLock.ReadUnlock();
		RwLock.WriteLock();
		return true;
	}

	return false;
}

// Base class for lock hierarchy. When used as a function parameter it means the called must hold at least a read lock
class FCollectionScopeLock
{
protected:
	UE_NODISCARD_CTOR explicit FCollectionScopeLock(FCollectionRecursiveRwLock& InLockObject, bool InWriteLock)
	: LockObject(InLockObject)
	, bWriteLock(InWriteLock)
	{
		if (InWriteLock)
		{
			LockObject.WriteLock();
		}
		else
		{
			LockObject.ReadLock();
		}
	}
	
	// Promote the lock from read to write, possibly being interrupted by another writer in between.
	// Returns false if the lock cannot be promoted, which can happen if the thread already holds a read
	// lock then reenters and tries to promote to a write lock.
	[[nodiscard]] bool PromoteInterruptible()
	{
		if (bWriteLock)
		{
			// Already promoted.
			return true;
		}

		if (LockObject.PromoteInterruptible())
		{
			bWriteLock = true;
		}

		return bWriteLock;
	}
	
	~FCollectionScopeLock()
	{
		if(bWriteLock)
		{
			LockObject.WriteUnlock();
		}
		else
		{
			LockObject.ReadUnlock();
		}
	}

	// Used for assertions to confirm that the correct kind of lock has been taken
	bool IsWriteLock()
	{
		return bWriteLock;
	}
	
private:
	UE_NONCOPYABLE(FCollectionScopeLock);
	FCollectionRecursiveRwLock& LockObject;
	bool bWriteLock = false;
};

// Scoped lock type used to hold lock and to tag methods which should at least hold a read lock 
class FCollectionScopeLock_Read : public FCollectionScopeLock
{
public:
	UE_NODISCARD_CTOR explicit FCollectionScopeLock_Read(FCollectionRecursiveRwLock& InLockObject)
	: FCollectionScopeLock(InLockObject, false)
	{
	}
};

// A lock on the collection container which begins in a read only state and can be promoted into a write lock with potential interruption in between 
class FCollectionScopeLock_RW : public FCollectionScopeLock
{
public:
	UE_NODISCARD_CTOR explicit FCollectionScopeLock_RW(FCollectionRecursiveRwLock& InLockObject, bool bWrite = false)
	: FCollectionScopeLock(InLockObject, bWrite)
	{
	}
	
	// Promoted the lock from read to write, possibly being interrupted by another writer in between
	using FCollectionScopeLock::PromoteInterruptible;
	// Used for assertions to confirm that the correct kind of lock has been taken
	using FCollectionScopeLock::IsWriteLock;
};

// Write lock on the collection container
class FCollectionScopeLock_Write : public FCollectionScopeLock_RW
{
public:
	UE_NODISCARD_CTOR explicit FCollectionScopeLock_Write(FCollectionRecursiveRwLock& InLockObject)
	: FCollectionScopeLock_RW(InLockObject, true)
	{
	}
};

/** Wraps up the lazy caching of the collection container */
class FCollectionContainerCache
{
public:
	FCollectionContainerCache(TMap<FCollectionNameType, TSharedRef<FCollection>>* InAvailableCollections);

	/** 
	 * Dirty the parts of the cache that need to change when a collection is added to our collection container.
	 * The collection container must be locked.
	 */
	void HandleCollectionAdded(FCollectionScopeLock_Write&);
	
	/** 
	 * Dirty the parts of the cache that need to change when a collection is removed from our collection container 
	 * The collection container must be locked.
	 */
	void HandleCollectionRemoved(FCollectionScopeLock_Write&);

	/** 
	 * Dirty the parts of the cache that need to change when a collection is modified 
	 * The collcetion container must be lockedl
	 */
	void HandleCollectionChanged(FCollectionScopeLock_Write&);

	/** 
	 * Update the given dirty parts of the cache based on which parts will be accessed while the given lock is held.
	 * A read/write lock will be promoted to a write lock if the cache must be updated.
	 * A write lock may also be passed as it extends the read/write lock.
	 * The calling thread may be interrupted by another write operation during the promotion operation.
	 * Therefore, caches should be updated as early as possible in order to prevent invalidation of state.
	 * 
	 * Returns false if the lock cannot be promoted preventing the cache from updating, which can happen if the
	 * thread already holds a read lock then reenters and tries to promote to a write lock.
	 * 
	 * This function is used rather than updating the caches in the Get* functions to prevent issues with pre-emption 
	 * on the lock upgrade deep into a method.
	 */
	bool UpdateCaches(FCollectionScopeLock_RW& InGuard, ECollectionCacheFlags Flags);

	/** 
	 * Access the CachedCollectionNamesFromGuids map, asserting that it is up-to-date.
	 * The collection container must be read-locked.
	 */
	const TMap<FGuid, FCollectionNameType>& GetCachedCollectionNamesFromGuids(FCollectionScopeLock&) const;

	/** 
	 * Access the CachedObjects map, asserting that it is up-to-date.
	 * The collection container must be read-locked.
	 */
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& GetCachedObjects(FCollectionScopeLock&) const;

	/** 
	 * Access the CachedHierarchy map, asserting that it is up-to-date.
	 * The collection container must be read-locked.
	 */
	const TMap<FGuid, TArray<FGuid>>& GetCachedHierarchy(FCollectionScopeLock&) const;

	/** 
	 * Access the CachedColors array, asserting that it is up-to-date 
	 * The collection container must be read-locked.
	 */
	const TArray<FLinearColor>& GetCachedColors(FCollectionScopeLock&) const;

	enum class ERecursiveWorkerFlowControl : uint8
	{
		Stop,
		Continue,
	};

	typedef TFunctionRef<ERecursiveWorkerFlowControl(const FCollectionNameType&, ECollectionRecursionFlags::Flag)> FRecursiveWorkerFunc;

	/** 
	 * Perform a recursive operation on the given collection and optionally its parents and children.
	 * The collection container must be read-locked and UpdateCaches must be called for names and hierarchy.
	 */
	void RecursionHelper_DoWork(FCollectionScopeLock&, const FCollectionNameType& InCollectionKey, const ECollectionRecursionFlags::Flags InRecursionMode, FRecursiveWorkerFunc InWorkerFunc) const;

private:
	/** Reference to the collections that are currently available in our owner collection container */
	TMap<FCollectionNameType, TSharedRef<FCollection>>* AvailableCollections;

	/** A map of collection GUIDs to their associated collection names */
	TMap<FGuid, FCollectionNameType> CachedCollectionNamesFromGuids_Internal;

	/** A map of object paths to their associated collection info - only objects that are in collections will appear in here */
	TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>> CachedObjects_Internal;

	/** A map of parent collection GUIDs to their child collection GUIDs - only collections that have children will appear in here */
	TMap<FGuid, TArray<FGuid>> CachedHierarchy_Internal;

	/** An array of all unique colors currently used by collections */
	TArray<FLinearColor> CachedColors_Internal;

	/** Which parts of the cache are dirty */
	ECollectionCacheFlags DirtyFlags = ECollectionCacheFlags::All;

	ERecursiveWorkerFlowControl RecursionHelper_DoWorkOnParents(FCollectionScopeLock&, const FCollectionNameType& InCollectionKey, FRecursiveWorkerFunc InWorkerFunc) const;
	ERecursiveWorkerFlowControl RecursionHelper_DoWorkOnChildren(FCollectionScopeLock&, const FCollectionNameType& InCollectionKey, FRecursiveWorkerFunc InWorkerFunc) const;
};

FCollectionContainerCache::FCollectionContainerCache(TMap<FCollectionNameType, TSharedRef<FCollection>>* InAvailableCollections)
	: AvailableCollections(InAvailableCollections)
{
}

void FCollectionContainerCache::HandleCollectionAdded(FCollectionScopeLock_Write&)
{
	DirtyFlags |= ECollectionCacheFlags::Names;
}

void FCollectionContainerCache::HandleCollectionRemoved(FCollectionScopeLock_Write&)
{
	DirtyFlags |= ECollectionCacheFlags::All;
}

void FCollectionContainerCache::HandleCollectionChanged(FCollectionScopeLock_Write&)
{
	DirtyFlags |= ECollectionCacheFlags::Objects | ECollectionCacheFlags::Hierarchy | ECollectionCacheFlags::Colors;
}

bool FCollectionContainerCache::UpdateCaches(FCollectionScopeLock_RW& InGuard, ECollectionCacheFlags ToUpdate)
{
	// Updating objects or hierarchy requires name mapping
	if (EnumHasAnyFlags(ToUpdate, ECollectionCacheFlags::Hierarchy | ECollectionCacheFlags::Objects))
	{
		ToUpdate |= ECollectionCacheFlags::Names;
	}

	// Updating objects requires hierarchy
	if (EnumHasAnyFlags(ToUpdate, ECollectionCacheFlags::Objects))
	{
		ToUpdate |= ECollectionCacheFlags::Hierarchy;
	}

	if (EnumHasAnyFlags(DirtyFlags, ToUpdate))
	{
		if (!InGuard.PromoteInterruptible())
		{
			// We assume get operations that require updating the cache on a thread we've reentered are originating from Slate bindings,
			// as such using a stale cache should be ok since such operations should succeed again when Slate ticks after the stack unwinds.
			return false;
		}
	}

	if (!EnumHasAnyFlags(DirtyFlags, ToUpdate))
	{
		// Caches we care about were updated while we switched locks
		return true;
	}

	// Limit updates to what's dirty 
	ToUpdate = ToUpdate & DirtyFlags;
	const double CacheStartTime = FPlatformTime::Seconds();

	if (EnumHasAllFlags(ToUpdate, ECollectionCacheFlags::Names))
	{
		CachedCollectionNamesFromGuids_Internal.Reset();
		EnumRemoveFlags(DirtyFlags, ECollectionCacheFlags::Names);
		for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : *AvailableCollections)
		{
			const FCollectionNameType& CollectionKey = AvailableCollection.Key;
			const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

			CachedCollectionNamesFromGuids_Internal.Add(Collection->GetCollectionGuid(), CollectionKey);
		}
	}

	if (EnumHasAllFlags(ToUpdate, ECollectionCacheFlags::Hierarchy))
	{
		CachedHierarchy_Internal.Reset();
		EnumRemoveFlags(DirtyFlags, ECollectionCacheFlags::Hierarchy);
		const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = GetCachedCollectionNamesFromGuids(InGuard);

		for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : *AvailableCollections)
		{
			const FCollectionNameType& CollectionKey = AvailableCollection.Key;
			const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

			// Make sure this is a known parent GUID before adding it to the map
			const FGuid& ParentCollectionGuid = Collection->GetParentCollectionGuid();
			if (CachedCollectionNamesFromGuids.Contains(ParentCollectionGuid))
			{
				TArray<FGuid>& CollectionChildren = CachedHierarchy_Internal.FindOrAdd(ParentCollectionGuid);
				CollectionChildren.AddUnique(Collection->GetCollectionGuid());
			}
		}
	}

	if (EnumHasAllFlags(ToUpdate, ECollectionCacheFlags::Objects))
	{
		CachedObjects_Internal.Reset();
		EnumRemoveFlags(DirtyFlags, ECollectionCacheFlags::Objects);

		for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : *AvailableCollections)
		{
			const FCollectionNameType& CollectionKey = AvailableCollection.Key;
			const TSharedRef<FCollection>& Collection = AvailableCollection.Value;
			const TSet<FSoftObjectPath>& ObjectsInCollection = Collection->GetObjectSet();

			if (ObjectsInCollection.Num() == 0)
			{
				continue;
			}

			auto RebuildCachedObjectsWorker = [CachedObjects_Internal=&CachedObjects_Internal, &ObjectsInCollection]
			(const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> ERecursiveWorkerFlowControl
			{
				// The worker reason will tell us why this collection is being processed (eg, because it is a parent of the collection we told it to DoWork on),
				// however, the reason this object exists in that parent collection is because a child collection contains it, and this is the reason we need
				// to put into the FObjectCollectionInfo, since that's what we'll test against later when we do the "do my children contain this object"? test
				// That's why we flip the reason logic here...
				ECollectionRecursionFlags::Flag ReasonObjectInCollection = InReason;
				switch (InReason)
				{
				case ECollectionRecursionFlags::Parents:
					ReasonObjectInCollection = ECollectionRecursionFlags::Children;
					break;
				case ECollectionRecursionFlags::Children:
					ReasonObjectInCollection = ECollectionRecursionFlags::Parents;
					break;
				default:
					break;
				}

				for (const FSoftObjectPath& ObjectPath : ObjectsInCollection)
				{
					TArray<FObjectCollectionInfo>& ObjectCollectionInfos = CachedObjects_Internal->FindOrAdd(ObjectPath);
					FObjectCollectionInfo* ObjectInfoPtr = ObjectCollectionInfos.FindByPredicate([InCollectionKey](FObjectCollectionInfo& InCollectionInfo) { return InCollectionInfo.CollectionKey == InCollectionKey; });
					if (ObjectInfoPtr)
					{
						ObjectInfoPtr->Reason |= ReasonObjectInCollection;
					}
					else
					{
						ObjectCollectionInfos.Add(FObjectCollectionInfo(InCollectionKey, ReasonObjectInCollection));
					}
				}
				return ERecursiveWorkerFlowControl::Continue;
			};

			// Recursively process all collections so that they know they contain these objects (and why!)
			RecursionHelper_DoWork(InGuard, CollectionKey, ECollectionRecursionFlags::All, RebuildCachedObjectsWorker);
		}

	}

	if (EnumHasAllFlags(ToUpdate, ECollectionCacheFlags::Colors))
	{
		CachedColors_Internal.Reset();
		EnumRemoveFlags(DirtyFlags, ECollectionCacheFlags::Colors);
		for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : *AvailableCollections)
		{
			const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

			if (const TOptional<FLinearColor> CollectionColor = Collection->GetCollectionColor())
			{
				CachedColors_Internal.Add(CollectionColor.GetValue());
			}
			// Deduplicate
			Algo::SortBy(CachedColors_Internal, [](FLinearColor Color) { return GetTypeHash(Color); });
			CachedColors_Internal.SetNum(Algo::Unique(CachedColors_Internal));
		}
	}

	UE_LOGF(LogCollectionManager, Verbose, "Rebuilt caches for %d collections in in %0.6f seconds", AvailableCollections->Num(), FPlatformTime::Seconds() - CacheStartTime);
	return true;
}

void FCollectionContainerCache::RecursionHelper_DoWork(FCollectionScopeLock& Guard,
	const FCollectionNameType& InCollectionKey,
	const ECollectionRecursionFlags::Flags InRecursionMode,
	FRecursiveWorkerFunc InWorkerFunc) const
{
	checkf(!EnumHasAnyFlags(DirtyFlags, ECollectionCacheFlags::RecursionWorker), TEXT("Collection cache must be updated with RecursionWorker flags before recursing through hierarchy."));

	if ((InRecursionMode & ECollectionRecursionFlags::Self) && InWorkerFunc(InCollectionKey, ECollectionRecursionFlags::Self) == ERecursiveWorkerFlowControl::Stop)
	{
		return;
	}

	if ((InRecursionMode & ECollectionRecursionFlags::Parents) && RecursionHelper_DoWorkOnParents(Guard, InCollectionKey, InWorkerFunc) == ERecursiveWorkerFlowControl::Stop)
	{
		return;
	}

	if ((InRecursionMode & ECollectionRecursionFlags::Children) && RecursionHelper_DoWorkOnChildren(Guard, InCollectionKey, InWorkerFunc) == ERecursiveWorkerFlowControl::Stop)
	{
		return;
	}
}

FCollectionContainerCache::ERecursiveWorkerFlowControl FCollectionContainerCache::RecursionHelper_DoWorkOnParents(
	FCollectionScopeLock& Guard, const FCollectionNameType& InCollectionKey, FRecursiveWorkerFunc InWorkerFunc) const
{
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
	if (CollectionRefPtr)
	{
		const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = GetCachedCollectionNamesFromGuids(Guard);

		const FCollectionNameType* const ParentCollectionKeyPtr = CachedCollectionNamesFromGuids.Find((*CollectionRefPtr)->GetParentCollectionGuid());
		if (ParentCollectionKeyPtr)
		{
			if (InWorkerFunc(*ParentCollectionKeyPtr, ECollectionRecursionFlags::Parents) == ERecursiveWorkerFlowControl::Stop || RecursionHelper_DoWorkOnParents(Guard, *ParentCollectionKeyPtr, InWorkerFunc) == ERecursiveWorkerFlowControl::Stop)
			{
				return ERecursiveWorkerFlowControl::Stop;
			}
		}
	}

	return ERecursiveWorkerFlowControl::Continue;
}

FCollectionContainerCache::ERecursiveWorkerFlowControl FCollectionContainerCache::RecursionHelper_DoWorkOnChildren(
	FCollectionScopeLock& Guard, const FCollectionNameType& InCollectionKey, FRecursiveWorkerFunc InWorkerFunc) const
{
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
	if (CollectionRefPtr)
	{
		const TMap<FGuid, TArray<FGuid>>& CachedHierarchy = GetCachedHierarchy(Guard);

		const TArray<FGuid>* const ChildCollectionGuids = CachedHierarchy.Find((*CollectionRefPtr)->GetCollectionGuid());
		if (ChildCollectionGuids)
		{
			for (const FGuid& ChildCollectionGuid : *ChildCollectionGuids)
			{
				const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = GetCachedCollectionNamesFromGuids(Guard);

				const FCollectionNameType* const ChildCollectionKeyPtr = CachedCollectionNamesFromGuids.Find(ChildCollectionGuid);
				if (ChildCollectionKeyPtr)
				{
					if (InWorkerFunc(*ChildCollectionKeyPtr, ECollectionRecursionFlags::Children) == ERecursiveWorkerFlowControl::Stop || RecursionHelper_DoWorkOnChildren(Guard, *ChildCollectionKeyPtr, InWorkerFunc) == ERecursiveWorkerFlowControl::Stop)
					{
						return ERecursiveWorkerFlowControl::Stop;
					}
				}
			}
		}
	}

	return ERecursiveWorkerFlowControl::Continue;
}

const TMap<FGuid, FCollectionNameType>& FCollectionContainerCache::GetCachedCollectionNamesFromGuids(FCollectionScopeLock&) const
{
	checkf(!EnumHasAnyFlags(DirtyFlags, ECollectionCacheFlags::Names), TEXT("Accessed guid->name map without updating cache"));
	return CachedCollectionNamesFromGuids_Internal;
}

const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& FCollectionContainerCache::GetCachedObjects(FCollectionScopeLock&) const
{
	checkf(!EnumHasAnyFlags(DirtyFlags, ECollectionCacheFlags::Objects), TEXT("Accessd object->collection map without updating cache"));
	return CachedObjects_Internal;
}

const TMap<FGuid, TArray<FGuid>>& FCollectionContainerCache::GetCachedHierarchy(FCollectionScopeLock&) const
{
	checkf(!EnumHasAnyFlags(DirtyFlags, ECollectionCacheFlags::Hierarchy), TEXT("Accessed collection hierarchy map without updating cache"));
	return CachedHierarchy_Internal;
}

const TArray<FLinearColor>& FCollectionContainerCache::GetCachedColors(FCollectionScopeLock&) const
{
	checkf(!EnumHasAnyFlags(DirtyFlags, ECollectionCacheFlags::Colors), TEXT("Accessed collection colors without updating cache"));
	return CachedColors_Internal;
}

FStringView FCollectionContainer::CollectionExtension = TEXTVIEW("collection");

FCollectionContainer::FCollectionContainer(FCollectionManager& InCollectionManager, const TSharedRef<ICollectionSource>& InCollectionSource)
	: CollectionManager(&InCollectionManager)
	, CollectionSource(InCollectionSource)
	, ReadOnlyFlags(0)
	, bIsHidden(false)
	, CollectionCache(MakePimpl<FCollectionContainerCache>(&AvailableCollections))
{
	LoadCollections();

	// Watch for changes that may happen outside of the collection container
	for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
	{
		const FString& CollectionFolder = CollectionSource->GetCollectionFolder(static_cast<ECollectionShareType::Type>(CacheIdx));

		if (CollectionFolder.IsEmpty())
		{
			continue;
		}

		// Make sure the folder we want to watch exists on disk
		if (!IFileManager::Get().MakeDirectory(*CollectionFolder, true))
		{
			continue;
		}

		DirectoryWatcher::FFileCacheConfig FileCacheConfig(FPaths::ConvertRelativePathToFull(CollectionFolder), FString());
		FileCacheConfig.DetectMoves(false);
		FileCacheConfig.RequireFileHashes(false);

		CollectionFileCaches[CacheIdx] = MakeShareable(new DirectoryWatcher::FFileCache(FileCacheConfig));
	}
}

FCollectionContainer::~FCollectionContainer() = default;

bool FCollectionContainer::IsReadOnly(ECollectionShareType::Type ShareType) const
{
	FCollectionScopeLock_Read Guard(Lock);

	return IsReadOnly(Guard, ShareType);
}

void FCollectionContainer::SetReadOnly(ECollectionShareType::Type ShareType, bool bReadOnly)
{
	check(ShareType <= ECollectionShareType::CST_All);

	FCollectionScopeLock_Write Guard(Lock);

	const uint8 ReadOnlyMask = GetReadOnlyMask(ShareType);
	if (bReadOnly)
	{
		ReadOnlyFlags |= ReadOnlyMask;
	}
	else
	{
		ReadOnlyFlags &= ~ReadOnlyMask;
	}
}

bool FCollectionContainer::IsHidden() const
{
	FCollectionScopeLock_Read Guard(Lock);
	return bIsHidden;
}

void FCollectionContainer::SetHidden(bool bHidden)
{
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (bIsHidden == bHidden)
		{
			return;
		}

		bIsHidden = bHidden;
	}

	IsHiddenChangedEvent.Broadcast(*this, bHidden);
}

bool FCollectionContainer::HasCollections() const
{
	FCollectionScopeLock_Read Guard(Lock);
	return AvailableCollections.Num() > 0;
}

void FCollectionContainer::GetCollections(TArray<FCollectionNameType>& OutCollections) const
{
	FCollectionScopeLock_Read Guard(Lock);
	OutCollections.Reserve(AvailableCollections.Num());
	for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : AvailableCollections)
	{
		const FCollectionNameType& CollectionKey = AvailableCollection.Key;
		OutCollections.Add(CollectionKey);
	}
}

void FCollectionContainer::GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const
{
	FCollectionScopeLock_Read Guard(Lock);
	for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
	{
		if (AvailableCollections.Contains(FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx))))
		{
			OutCollections.Add(FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx)));
		}
	}
}

void FCollectionContainer::GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const
{
	FCollectionScopeLock_Read Guard(Lock);
	for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : AvailableCollections)
	{
		const FCollectionNameType& CollectionKey = AvailableCollection.Key;
		if (ShareType == ECollectionShareType::CST_All || ShareType == CollectionKey.Type)
		{
			CollectionNames.AddUnique(CollectionKey.Name);
		}
	}
}

void FCollectionContainer::GetRootCollections(TArray<FCollectionNameType>& OutCollections) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names);
	const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);

	OutCollections.Reserve(AvailableCollections.Num());
	for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : AvailableCollections)
	{
		const FCollectionNameType& CollectionKey = AvailableCollection.Key;
		const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

		// A root collection either has no parent GUID, or a parent GUID that cannot currently be found - the check below handles both
		if (!CachedCollectionNamesFromGuids.Contains(Collection->GetParentCollectionGuid()))
		{
			OutCollections.Add(CollectionKey);
		}
	}
}

void FCollectionContainer::GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names);
	const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);

	for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : AvailableCollections)
	{
		const FCollectionNameType& CollectionKey = AvailableCollection.Key;
		const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

		if (ShareType == ECollectionShareType::CST_All || ShareType == CollectionKey.Type)
		{
			// A root collection either has no parent GUID, or a parent GUID that cannot currently be found - the check below handles both
			if (!CachedCollectionNamesFromGuids.Contains(Collection->GetParentCollectionGuid()))
			{
				CollectionNames.AddUnique(CollectionKey.Name);
			}
		}
	}
}

void FCollectionContainer::GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names | ECollectionCacheFlags::Hierarchy);

	const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);
	const TMap<FGuid, TArray<FGuid>>& CachedHierarchy = CollectionCache->GetCachedHierarchy(Guard);

	const int32 Start = ShareType == ECollectionShareType::CST_All ? 0 : int32(ShareType);
	const int32 End = ShareType == ECollectionShareType::CST_All ? int32(ECollectionShareType::CST_All) : int32(ShareType) + 1;

	for (int32 CacheIdx = Start; CacheIdx < End; ++CacheIdx)
	{
		FCollectionNameType CollectionKey{ CollectionName, ECollectionShareType::Type(CacheIdx) };
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			continue;
		}

		const TArray<FGuid>* ChildCollectionGuids = CachedHierarchy.Find((*CollectionRefPtr)->GetCollectionGuid());
		if (!ChildCollectionGuids)
		{
			continue;
		}

		for (const FGuid& ChildCollectionGuid : *ChildCollectionGuids)
		{
			const FCollectionNameType* const ChildCollectionKeyPtr = CachedCollectionNamesFromGuids.Find(ChildCollectionGuid);
			if (ChildCollectionKeyPtr)
			{
				OutCollections.Add(*ChildCollectionKeyPtr);
			}
		}
	};
}

void FCollectionContainer::GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names | ECollectionCacheFlags::Hierarchy);
	const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);
	const TMap<FGuid, TArray<FGuid>>& CachedHierarchy = CollectionCache->GetCachedHierarchy(Guard);

	const int32 Start = ShareType == ECollectionShareType::CST_All ? 0 : int32(ShareType);
	const int32 End = ShareType == ECollectionShareType::CST_All ? int32(ECollectionShareType::CST_All) : int32(ShareType) + 1;

	for (int32 CacheIdx = Start; CacheIdx < End; ++CacheIdx)
	{
		FCollectionNameType CollectionKey{ CollectionName, ECollectionShareType::Type(CacheIdx) };
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			continue;
		}

		const TArray<FGuid>* ChildCollectionGuids = CachedHierarchy.Find((*CollectionRefPtr)->GetCollectionGuid());
		if (!ChildCollectionGuids)
		{
			continue;
		}

		for (const FGuid& ChildCollectionGuid : *ChildCollectionGuids)
		{
			const FCollectionNameType* const ChildCollectionKeyPtr = CachedCollectionNamesFromGuids.Find(ChildCollectionGuid);
			if (ChildCollectionKeyPtr && (ChildShareType == ECollectionShareType::CST_All || ChildShareType == ChildCollectionKeyPtr->Type))
			{
				CollectionNames.AddUnique(ChildCollectionKeyPtr->Name);
			}
		}
	};
}

TOptional<FCollectionNameType> FCollectionContainer::GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const
{
	FCollectionScopeLock_RW Guard(Lock);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(FCollectionNameType(CollectionName, ShareType));
	if (!CollectionRefPtr)
	{
		return TOptional<FCollectionNameType>();
	}

	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names);
	const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);
	const FCollectionNameType* const ParentCollectionKeyPtr = CachedCollectionNamesFromGuids.Find((*CollectionRefPtr)->GetParentCollectionGuid());
	if (ParentCollectionKeyPtr)
	{
		return *ParentCollectionKeyPtr;
	}
	return TOptional<FCollectionNameType>();
}

bool FCollectionContainer::CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const
{
	FCollectionScopeLock_Read Guard(Lock);
	return CollectionExists_Locked(Guard, CollectionName, ShareType);
}

bool FCollectionContainer::CollectionExists_Locked(FCollectionScopeLock&, FName CollectionName, ECollectionShareType::Type ShareType) const
{
	if (ShareType == ECollectionShareType::CST_All)
	{
		// Asked to check all share types...
		for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
		{
			if (AvailableCollections.Contains(FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx))))
			{
				// Collection exists in at least one cache
				return true;
			}
		}

		// Collection not found in any cache
		return false;
	}
	else
	{
		return AvailableCollections.Contains(FCollectionNameType(CollectionName, ShareType));
	}
}

bool FCollectionContainer::GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& AssetsPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker);
	bool bFoundAssets = false;

	auto GetAssetsInCollectionWorker = [AvailableCollections=&AvailableCollections, &AssetsPaths, &bFoundAssets](const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
	{
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
		if (CollectionRefPtr)
		{
			(*CollectionRefPtr)->GetAssetsInCollection(AssetsPaths);
			bFoundAssets = true;
		}
		return FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
	};

	if (ShareType == ECollectionShareType::CST_All)
	{
		// Asked for all share types, find assets in the specified collection name in any cache
		for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
		{
			CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx)), RecursionMode, GetAssetsInCollectionWorker);
		}
	}
	else
	{
		CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ShareType), RecursionMode, GetAssetsInCollectionWorker);
	}

	return bFoundAssets;
}

bool FCollectionContainer::GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker);
	bool bFoundClasses = false;

	auto GetClassesInCollectionWorker = [AvailableCollections=&AvailableCollections, &ClassPaths, &bFoundClasses](const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
	{
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
		if (CollectionRefPtr)
		{
			(*CollectionRefPtr)->GetClassesInCollection(ClassPaths);
			bFoundClasses = true;
		}
		return FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
	};

	if (ShareType == ECollectionShareType::CST_All)
	{
		// Asked for all share types, find classes in the specified collection name in any cache
		for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
		{
			CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx)), RecursionMode, GetClassesInCollectionWorker);
		}
	}
	else
	{
		CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ShareType), RecursionMode, GetClassesInCollectionWorker);
	}

	return bFoundClasses;
}

bool FCollectionContainer::GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker);
	bool bFoundObjects = false;

	auto GetObjectsInCollectionWorker = [AvailableCollections=&AvailableCollections, &ObjectPaths, &bFoundObjects](const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
	{
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
		if (CollectionRefPtr)
		{
			(*CollectionRefPtr)->GetObjectsInCollection(ObjectPaths);
			bFoundObjects = true;
		}
		return FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
	};

	if (ShareType == ECollectionShareType::CST_All)
	{
		// Asked for all share types, find classes in the specified collection name in any cache
		for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
		{
			CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ECollectionShareType::Type(CacheIdx)), RecursionMode, GetObjectsInCollectionWorker);
		}
	}
	else
	{
		CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ShareType), RecursionMode, GetObjectsInCollectionWorker);
	}

	return bFoundObjects;
}

void FCollectionContainer::GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects);
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);

	const TArray<FObjectCollectionInfo>* ObjectCollectionInfosPtr = CachedObjects.Find(ObjectPath);
	if (ObjectCollectionInfosPtr)
	{
		for (const FObjectCollectionInfo& ObjectCollectionInfo : *ObjectCollectionInfosPtr)
		{
			if ((ShareType == ECollectionShareType::CST_All || ShareType == ObjectCollectionInfo.CollectionKey.Type) && (RecursionMode & ObjectCollectionInfo.Reason) != 0)
			{
				OutCollectionNames.Add(ObjectCollectionInfo.CollectionKey.Name);
			}
		}
	}
}

void FCollectionContainer::GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects);
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);

	const TArray<FObjectCollectionInfo>* ObjectCollectionInfosPtr = CachedObjects.Find(ObjectPath);
	if (ObjectCollectionInfosPtr)
	{
		OutCollections.Reserve(OutCollections.Num() + ObjectCollectionInfosPtr->Num());
		for (const FObjectCollectionInfo& ObjectCollectionInfo : *ObjectCollectionInfosPtr)
		{
			if ((RecursionMode & ObjectCollectionInfo.Reason) != 0)
			{
				OutCollections.Add(ObjectCollectionInfo.CollectionKey);
			}
		}
	}
}

void FCollectionContainer::GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects);
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);

	for (const FSoftObjectPath& ObjectPath : ObjectPaths)
	{
		const TArray<FObjectCollectionInfo>* ObjectCollectionInfosPtr = CachedObjects.Find(ObjectPath);
		if (ObjectCollectionInfosPtr)
		{
			for (const FObjectCollectionInfo& ObjectCollectionInfo : *ObjectCollectionInfosPtr)
			{
				if ((RecursionMode & ObjectCollectionInfo.Reason) != 0)
				{
					TArray<FSoftObjectPath>& MatchedObjects = OutCollectionsAndMatchedObjects.FindOrAdd(ObjectCollectionInfo.CollectionKey);
					MatchedObjects.Add(ObjectPath);
				}
			}
		}
	}
}

FString FCollectionContainer::GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode, bool bFullPaths) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects | ECollectionCacheFlags::RecursionWorker);
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);

	const TArray<FObjectCollectionInfo>* ObjectCollectionInfosPtr = CachedObjects.Find(ObjectPath);
	if (ObjectCollectionInfosPtr)
	{
		TArray<FString> CollectionNameStrings;
		TArray<FString> CollectionPathStrings;

		auto GetCollectionsStringForObjectWorker = [&CollectionPathStrings](const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
		{
			CollectionPathStrings.Insert(InCollectionKey.Name.ToString(), 0);
			return FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
		};

		for (const FObjectCollectionInfo& ObjectCollectionInfo : *ObjectCollectionInfosPtr)
		{
			if ((ShareType == ECollectionShareType::CST_All || ShareType == ObjectCollectionInfo.CollectionKey.Type) && (RecursionMode & ObjectCollectionInfo.Reason) != 0)
			{
				if (bFullPaths)
				{
					CollectionPathStrings.Reset();
					CollectionCache->RecursionHelper_DoWork(Guard, ObjectCollectionInfo.CollectionKey, ECollectionRecursionFlags::SelfAndParents, GetCollectionsStringForObjectWorker);
					CollectionNameStrings.Add(FString::Join(CollectionPathStrings, TEXT("/")));
				}
				else
				{
					CollectionNameStrings.Add(ObjectCollectionInfo.CollectionKey.Name.ToString());
				}
			}
		}

		if (CollectionNameStrings.Num() > 0)
		{
			CollectionNameStrings.Sort();
			return FString::Join(CollectionNameStrings, TEXT(", "));
		}
	}

	return FString();
}

FString FCollectionContainer::MakeCollectionPath(FName CollectionName, ECollectionShareType::Type ShareType) const
{
	if (ShareType == ECollectionShareType::CST_All)
	{
		return FString::Printf(TEXT("/%s/%s"), *CollectionSource->GetName().ToString(), *CollectionName.ToString());
	}
	else
	{
		return FString::Printf(TEXT("/%s/%s/%s"), *CollectionSource->GetName().ToString(), ECollectionShareType::ToString(ShareType), *CollectionName.ToString());
	}
}

void FCollectionContainer::CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const
{
	FCollectionScopeLock_Read Guard(Lock);
	int32 IntSuffix = 1;
	bool CollectionAlreadyExists = false;
	do
	{
		if (IntSuffix <= 1)
		{
			OutCollectionName = BaseName;
		}
		else
		{
			OutCollectionName = *FString::Printf(TEXT("%s%d"), *BaseName.ToString(), IntSuffix);
		}

		CollectionAlreadyExists = CollectionExists_Locked(Guard, OutCollectionName, ShareType);
		++IntSuffix;
	}
	while (CollectionAlreadyExists);
}

bool FCollectionContainer::IsValidCollectionName(const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError) const
{
	// Make sure we are not creating an FName that is too large
	if (CollectionName.Len() >= NAME_SIZE)
	{
		if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_CollectionNameTooLong","This collection name is too long ({0} characters), the maximum is {1}. Please choose a shorter name. Collection name: {2}"),
				FText::AsNumber(CollectionName.Len()), FText::AsNumber(NAME_SIZE), FText::FromString(CollectionName));
		}
		return false;
	}

	const FName CollectionNameFinal = *CollectionName;

	// Make sure the we actually have a new name set
	if (CollectionNameFinal.IsNone())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_CollectionNameEmptyOrNone", "This collection name cannot be empty or 'None'.");
		}
		return false;
	}

	// Make sure the new name only contains valid characters
	if (!CollectionNameFinal.IsValidXName(INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, OutError))
	{
		return false;
	}

	// Make sure we're not duplicating an existing collection name
	// NB: Ok to call public function here because we don't need acquire a lock for the previous checks 
	if (CollectionExists(CollectionNameFinal, ShareType))
	{
		if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_CollectionAlreadyExists", "A collection already exists with the name '{0}'."), FText::FromName(CollectionNameFinal));
		}
		return false;
	}

	return true;
}

bool FCollectionContainer::CreateCollection(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode, FText* OutError)
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	if (!IsValidCollectionName(CollectionName.ToString(), ShareType, OutError))
	{
		return false;
	}

	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		// Try to add the collection
		const bool bUseSCC = ShouldUseSCC(ShareType);
		const FString CollectionFilename = GetCollectionFilename(CollectionName, ShareType);

		// Validate collection name as file name
		FText UnusedError;
		bool bFilenameValid = FFileHelper::IsFilenameValidForSaving(CollectionName.ToString(), OutError ? *OutError : UnusedError);
		if (!bFilenameValid)
		{
			return false;
		}

		TSharedRef<FCollection> NewCollection = MakeShareable(new FCollection(CollectionFilename, bUseSCC, StorageMode));
		if (!AddCollection(Guard, NewCollection, ShareType))
		{
			// Failed to add the collection, it already exists
			if (OutError)
			{
				*OutError = LOCTEXT("Error_AlreadyExists", "The collection already exists.");
			}
			return false;
		}

		constexpr bool bForceCommitToRevisionControl = true;
		if (!InternalSaveCollection(Guard, NewCollection, OutError, bForceCommitToRevisionControl))
		{
			// Collection failed to save, remove it from the cache
			RemoveCollection(Guard, NewCollection, ShareType);
			return false;
		}

		CollectionFileCaches[ShareType]->IgnoreNewFile(NewCollection->GetSourceFilename());
	}

	// Collection saved!
	// Broadcast events outside of lock 
	CollectionCreatedEvent.Broadcast(*this, FCollectionNameType(CollectionName, ShareType));
	return true;
}

bool FCollectionContainer::RenameCollection(FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName, ECollectionShareType::Type NewShareType, FText* OutError)
{
	const FCollectionNameType OriginalCollectionKey(CurrentCollectionName, CurrentShareType);
	const FCollectionNameType NewCollectionKey(NewCollectionName, NewShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, CurrentShareType, OutError) || !ValidateWritable(Guard, NewShareType, OutError))
		{
			return false;
		}

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(OriginalCollectionKey);
		if (!CollectionRefPtr)
		{
			// The collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		// Add the new collection
		TSharedPtr<FCollection> NewCollection;
		{
			const bool bUseSCC = ShouldUseSCC(NewShareType);
			const FString NewCollectionFilename = GetCollectionFilename(NewCollectionName, NewShareType);

			// Create an exact copy of the collection using its new path - this will preserve its GUID and avoid losing hierarchy data
			NewCollection = (*CollectionRefPtr)->Clone(NewCollectionFilename, bUseSCC, ECollectionCloneMode::Exact);
			if (!AddCollection(Guard, NewCollection.ToSharedRef(), NewShareType))
			{
				// Failed to add the collection, it already exists
				if (OutError)
				{
					*OutError = LOCTEXT("Error_AlreadyExists", "The collection already exists.");
				}
				return false;
			}

			bool bForceCommitToRevisionControl = true;
			if (!InternalSaveCollection(Guard, NewCollection.ToSharedRef(), OutError, bForceCommitToRevisionControl))
			{
				// Collection failed to save, remove it from the cache
				RemoveCollection(Guard, NewCollection.ToSharedRef(), NewShareType);
				return false;
			}
		}

		// Remove the old collection
		{
			FText UnusedError;
			if ((*CollectionRefPtr)->DeleteSourceFile(OutError ? *OutError : UnusedError))
			{
				CollectionFileCaches[CurrentShareType]->IgnoreDeletedFile((*CollectionRefPtr)->GetSourceFilename());

				RemoveCollection(Guard, *CollectionRefPtr, CurrentShareType);
			}
			else
			{
				// Failed to remove the old collection, so remove the collection we created.
				NewCollection->DeleteSourceFile(OutError ? *OutError : UnusedError);
				RemoveCollection(Guard, NewCollection.ToSharedRef(), NewShareType);
				return false;
			}
		}

		CollectionFileCaches[NewShareType]->IgnoreNewFile(NewCollection->GetSourceFilename());

		CollectionCache->HandleCollectionChanged(Guard);
	}

	// Success, broadcast events outside of lock 
	CollectionRenamedEvent.Broadcast(*this, OriginalCollectionKey, NewCollectionKey);
	return true;
}

bool FCollectionContainer::ReparentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) 
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	TOptional<FCollectionNameType> OldParentCollectionKey;
	TOptional<FCollectionNameType> NewParentCollectionKey;
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError) || (!ParentCollectionName.IsNone() && !ValidateWritable(Guard, ParentShareType, OutError)))
		{
			return false;
		}

		verifyf(CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker), TEXT("UpdateCaches must be called within a write lock to guarantee subsequent usage to function as expected"));

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// The collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		const FGuid OldParentGuid = (*CollectionRefPtr)->GetParentCollectionGuid();
		FGuid NewParentGuid;

		if (!ParentCollectionName.IsNone())
		{
			// Find and set the new parent GUID
			NewParentCollectionKey = FCollectionNameType(ParentCollectionName, ParentShareType);
			TSharedRef<FCollection>* const ParentCollectionRefPtr = AvailableCollections.Find(NewParentCollectionKey.GetValue());
			if (!ParentCollectionRefPtr)
			{
				// The collection doesn't exist
				if (OutError)
				{
					*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
				}
				return false;
			}

			// Does the parent collection need saving in order to have a stable GUID?
			if ((*ParentCollectionRefPtr)->GetCollectionVersion() < ECollectionVersion::AddedCollectionGuid)
			{
				bool bForceCommitToRevisionControl = false;
				// Try and re-save the parent collection now
				if (InternalSaveCollection(Guard, *ParentCollectionRefPtr, OutError, bForceCommitToRevisionControl))
				{
					CollectionFileCaches[ParentShareType]->IgnoreFileModification((*ParentCollectionRefPtr)->GetSourceFilename());
				}
				else
				{
					return false;
				}
			}

			if (!IsValidParentCollection_Locked(Guard, CollectionName, ShareType, ParentCollectionName, ParentShareType, OutError))
			{
				return false;
			}

			NewParentGuid = (*ParentCollectionRefPtr)->GetCollectionGuid();
		}

		// Anything changed?
		if (OldParentGuid == NewParentGuid)
		{
			return true;
		}

		(*CollectionRefPtr)->SetParentCollectionGuid(NewParentGuid);

		// Try and save with the new parent GUID
		bool bForceCommitToRevisionControl = false;
		if (InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());
		}
		else
		{
			// Failed to save... rollback the collection to use its old parent GUID
			(*CollectionRefPtr)->SetParentCollectionGuid(OldParentGuid);
			return false;
		}

		CollectionCache->HandleCollectionChanged(Guard);
		verifyf(CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Names), TEXT("UpdateCaches must be called within a write lock to guarantee subsequent usage to function as expected"));

		// Find the old parent so we can notify about the change
		{
			const TMap<FGuid, FCollectionNameType>& CachedCollectionNamesFromGuids = CollectionCache->GetCachedCollectionNamesFromGuids(Guard);

			const FCollectionNameType* const OldParentCollectionKeyPtr = CachedCollectionNamesFromGuids.Find(OldParentGuid);
			if (OldParentCollectionKeyPtr)
			{
				OldParentCollectionKey = *OldParentCollectionKeyPtr;
			}
		}
	}

	// Success, broadcast event outside of lock
	CollectionReparentedEvent.Broadcast(*this, CollectionKey, OldParentCollectionKey, NewParentCollectionKey);
	return true;
}

bool FCollectionContainer::DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// The collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		FText UnusedError;
		if ((*CollectionRefPtr)->DeleteSourceFile(OutError ? *OutError : UnusedError))
		{
			CollectionFileCaches[ShareType]->IgnoreDeletedFile((*CollectionRefPtr)->GetSourceFilename());

			RemoveCollection(Guard, *CollectionRefPtr, ShareType);
		}
		else
		{
			// Failed to delete the source file
			return false;
		}
	}

	// Broadcast event outside of lock
	CollectionDestroyedEvent.Broadcast(*this, CollectionKey);
	return true;
}

bool FCollectionContainer::AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError)
{
	return AddToCollection(CollectionName, ShareType, MakeArrayView(&ObjectPath, 1), nullptr, OutError);
}

bool FCollectionContainer::AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumAdded, FText* OutError)
{
	if (OutNumAdded)
	{
		*OutNumAdded = 0;
	}

	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// Collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		if ((*CollectionRefPtr)->GetStorageMode() != ECollectionStorageMode::Static)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_AddNeedsStaticCollection", "Objects can only be added to static collections.");
			}
			return false;
		}
		
		int32 NumAdded = 0;
		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			if ((*CollectionRefPtr)->AddObjectToCollection(ObjectPath))
			{
				NumAdded++;
			}
		}

		if (NumAdded > 0)
		{
			constexpr bool bForceCommitToRevisionControl = false;
			if (InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
			{
				CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());

				// Added and saved
				if (OutNumAdded)
				{
					*OutNumAdded = NumAdded;
				}

				CollectionCache->HandleCollectionChanged(Guard);
				// Fall out of scope to return 
			}
			else
			{
				// Added but not saved, revert the add
				for (const FSoftObjectPath& ObjectPath : ObjectPaths)
				{
					(*CollectionRefPtr)->RemoveObjectFromCollection(ObjectPath);
				}
				return false;
			}
		}
		else
		{
			// Failed to add, all of the objects were already in the collection
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("Error_AlreadyInCollection", "The selected {0}|plural(one=item,other=items) {0}|plural(one=has,other=have) already been added to '{1}'"), ObjectPaths.Num(), FText::FromName(CollectionName));
			}
			return false;
		}
	}
	   
	// Broadast event out of lock
	AssetsAddedToCollectionDelegate.Broadcast(*this, CollectionKey, ObjectPaths);
	return true;
}

bool FCollectionContainer::RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError)
{
	return RemoveFromCollection(CollectionName, ShareType, MakeArrayView(&ObjectPath, 1), nullptr, OutError);
}

bool FCollectionContainer::RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumRemoved, FText* OutError)
{
	if (OutNumRemoved)
	{
		*OutNumRemoved = 0;
	}

	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// Collection not found
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		if ((*CollectionRefPtr)->GetStorageMode() != ECollectionStorageMode::Static)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_RemoveNeedsStaticCollection", "Objects can only be removed from static collections.");
			}
			return false;
		}
		
		TArray<FSoftObjectPath> RemovedAssets;
		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			if ((*CollectionRefPtr)->RemoveObjectFromCollection(ObjectPath))
			{
				RemovedAssets.Add(ObjectPath);
			}
		}

		if (RemovedAssets.Num() == 0)
		{
			// Failed to remove, none of the objects were in the collection
			if (OutError)
			{
				*OutError = LOCTEXT("Error_NotInCollection", "None of the assets were in the collection.");
			}
			return false;
		}

		constexpr bool bForceCommitToRevisionControl = false;
		if (!InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			// Removed but not saved, revert the remove
			for (const FSoftObjectPath& RemovedAssetName : RemovedAssets)
			{
				(*CollectionRefPtr)->AddObjectToCollection(RemovedAssetName);
			}
			return false;
		}

		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());

		// Removed and saved
		if (OutNumRemoved)
		{
			*OutNumRemoved = RemovedAssets.Num();
		}

		CollectionCache->HandleCollectionChanged(Guard);
	}

	// Broadcast event out of lock
	AssetsRemovedFromCollectionDelegate.Broadcast(*this, CollectionKey, ObjectPaths);
	return true;
}

bool FCollectionContainer::SetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// Collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		if ((*CollectionRefPtr)->GetStorageMode() != ECollectionStorageMode::Dynamic)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_SetNeedsDynamicCollection", "Search queries can only be set on dynamic collections.");
			}
			return false;
		}

		(*CollectionRefPtr)->SetDynamicQueryText(InQueryText);
		
		constexpr bool bForceCommitToRevisionControl = true;
		if (!InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			return false;
		}
		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());
		CollectionCache->HandleCollectionChanged(Guard);
	}

	// Broadcast event outside of lock
	CollectionUpdatedEvent.Broadcast(*this, CollectionKey);
	return true;

}

bool FCollectionContainer::GetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_Read Guard(Lock);
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
	if (!CollectionRefPtr)
	{
		// Collection doesn't exist
		if (OutError)
		{
			*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
		}
		return false;
	}

	if ((*CollectionRefPtr)->GetStorageMode() != ECollectionStorageMode::Dynamic)
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_GetNeedsDynamicCollection", "Search queries can only be got from dynamic collections.");
		}
		return false;
	}

	OutQueryText = (*CollectionRefPtr)->GetDynamicQueryText();
	return true;
}

bool FCollectionContainer::TestDynamicQuery(FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext, bool& OutResult, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_Read Guard(Lock);
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
	if (!CollectionRefPtr)
	{
		// Collection doesn't exist
		if (OutError)
		{
			*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
		}
		return false;
	}

	if ((*CollectionRefPtr)->GetStorageMode() != ECollectionStorageMode::Dynamic)
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_TestNeedsDynamicCollection", "Search queries can only be tested on dynamic collections.");
		}
		return false;
	}

	(*CollectionRefPtr)->PrepareDynamicQuery();
	OutResult = (*CollectionRefPtr)->TestDynamicQuery(InContext);
	return true;
}

bool FCollectionContainer::EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			// Collection doesn't exist
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		if ((*CollectionRefPtr)->IsEmpty())
		{
			// Already empty - nothing to do
			return true;
		}

		(*CollectionRefPtr)->Empty();
		
		constexpr bool bForceCommitToRevisionControl = true;
		if (!InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			return false;
		}
		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());

		CollectionCache->HandleCollectionChanged(Guard);
	}

	// Broadcast event outside of lock
	CollectionUpdatedEvent.Broadcast(*this, CollectionKey);
	return true;
}

bool FCollectionContainer::SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		FCollectionStatusInfo StatusInfo = (*CollectionRefPtr)->GetStatusInfo(IsReadOnly(Guard, ShareType));

		const bool bNeedsSave = StatusInfo.bIsDirty || (StatusInfo.SCCState.IsValid() && StatusInfo.SCCState->IsModified());
		if (!bNeedsSave)
		{
			// No changes - nothing to save
			return true;
		}

		constexpr bool bForceCommitToRevisionControl = true;
		if (!InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			return false;
		}

		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());

		CollectionCache->HandleCollectionChanged(Guard);
	}
	
	// Broadcast event out of lock
	CollectionUpdatedEvent.Broadcast(*this, CollectionKey);
	return true;
}

bool FCollectionContainer::UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		FText UnusedError;
		if (!(*CollectionRefPtr)->Update(OutError ? *OutError : UnusedError))
		{
			return false;
		}

		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());
		CollectionCache->HandleCollectionChanged(Guard);
	}

	// Broadcast event outside of lock
	CollectionUpdatedEvent.Broadcast(*this, CollectionKey);
	return true;
}

bool FCollectionContainer::GetCollectionStatusInfo(FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo, FText* OutError) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionContainer::GetCollectionStatusInfo);

	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_Read Guard(Lock);
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
	if (CollectionRefPtr)
	{
		OutStatusInfo = (*CollectionRefPtr)->GetStatusInfo(IsReadOnly(Guard, ShareType));
		return true;
	}
	else
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
		}
	}

	return false;
}

bool FCollectionContainer::HasCollectionColors(TArray<FLinearColor>* OutColors) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Colors);
	const TArray<FLinearColor>& CollectionColors = CollectionCache->GetCachedColors(Guard);
	if (OutColors)
	{
		*OutColors = CollectionColors;
	}
	return CollectionColors.Num() > 0;
}

bool FCollectionContainer::GetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_Read Guard(Lock);
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
	if (CollectionRefPtr)
	{
		OutColor = (*CollectionRefPtr)->GetCollectionColor();
		return true;
	}
	else
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
		}
	}

	return false;
}

bool FCollectionContainer::SetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor, FText* OutError)
{
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	{
		FCollectionScopeLock_Write Guard(Lock);

		if (!ValidateWritable(Guard, ShareType, OutError))
		{
			return false;
		}

		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
		if (!CollectionRefPtr)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
			}
			return false;
		}

		if ((*CollectionRefPtr)->GetCollectionColor() == NewColor)
		{
			return true;
		}

		(*CollectionRefPtr)->SetCollectionColor(NewColor);

		constexpr bool bForceCommitToRevisionControl = false;
		if (!InternalSaveCollection(Guard, *CollectionRefPtr, OutError, bForceCommitToRevisionControl))
		{
			return false;
		}
		
		CollectionFileCaches[ShareType]->IgnoreFileModification((*CollectionRefPtr)->GetSourceFilename());
		
		CollectionCache->HandleCollectionChanged(Guard);
	}
	
	// Broadcast event outside of lock 
	CollectionUpdatedEvent.Broadcast(*this, CollectionKey);
	return true;
}

bool FCollectionContainer::GetCollectionStorageMode(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_Read Guard(Lock);
	const FCollectionNameType CollectionKey(CollectionName, ShareType);
	const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey);
	if (CollectionRefPtr)
	{
		OutStorageMode = (*CollectionRefPtr)->GetStorageMode();
		return true;
	}
	else
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_DoesntExist", "The collection doesn't exist.");
		}
	}

	return false;
}

bool FCollectionContainer::IsObjectInCollection(const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType,ECollectionRecursionFlags::Flags RecursionMode, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker);
	bool bFoundObject = false;

	auto IsObjectInCollectionWorker = [AvailableCollections=&AvailableCollections, ObjectPath, &bFoundObject](const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
	{
		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
		if (CollectionRefPtr)
		{
			bFoundObject = (*CollectionRefPtr)->IsObjectInCollection(ObjectPath);
		}
		return (bFoundObject) ? FCollectionContainerCache::ERecursiveWorkerFlowControl::Stop : FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
	};

	CollectionCache->RecursionHelper_DoWork(Guard, FCollectionNameType(CollectionName, ShareType), RecursionMode, IsObjectInCollectionWorker);

	return bFoundObject;
}

bool FCollectionContainer::IsValidParentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName,ECollectionShareType::Type ParentShareType, FText* OutError) const
{
	FCollectionScopeLock_RW Guard(Lock);
	CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::RecursionWorker);
	return IsValidParentCollection_Locked(Guard, CollectionName, ShareType, ParentCollectionName, ParentShareType, OutError);
}

bool FCollectionContainer::IsValidParentCollection_Locked(FCollectionScopeLock& InGuard, FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All) || (!ParentCollectionName.IsNone() && !ensure(ParentShareType < ECollectionShareType::CST_All)))
	{
		// Bad share type
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	if (ParentCollectionName.IsNone())
	{
		// Clearing the parent is always valid
		return true;
	}

	bool bValidParent = true;
	auto IsValidParentCollectionWorker = [OutError, CollectionName, ShareType, &bValidParent, AvailableCollections=&AvailableCollections]
	(const FCollectionNameType& InCollectionKey, ECollectionRecursionFlags::Flag InReason) -> FCollectionContainerCache::ERecursiveWorkerFlowControl
	{
		const bool bMatchesCollectionBeingReparented = (CollectionName == InCollectionKey.Name && ShareType == InCollectionKey.Type);
		if (bMatchesCollectionBeingReparented)
		{
			bValidParent = false;
			if (OutError)
			{
				*OutError = (InReason == ECollectionRecursionFlags::Self)
						? LOCTEXT("InvalidParent_CannotParentToSelf", "A collection cannot be parented to itself")
						: LOCTEXT("InvalidParent_CannotParentToChildren", "A collection cannot be parented to its children");
			}
			return FCollectionContainerCache::ERecursiveWorkerFlowControl::Stop;
		}

		const bool bIsValidChildType = ECollectionShareType::IsValidChildType(InCollectionKey.Type, ShareType);
		if (!bIsValidChildType)
		{
			bValidParent = false;
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("InvalidParent_InvalidChildType", "A {0} collection cannot contain a {1} collection"), ECollectionShareType::ToText(InCollectionKey.Type), ECollectionShareType::ToText(ShareType));
			}
			return FCollectionContainerCache::ERecursiveWorkerFlowControl::Stop;
		}

		const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections->Find(InCollectionKey);
		if (CollectionRefPtr)
		{
			const ECollectionStorageMode::Type StorageMode = (*CollectionRefPtr)->GetStorageMode();
			if (StorageMode == ECollectionStorageMode::Dynamic)
			{
				bValidParent = false;
				if (OutError)
				{
					*OutError = LOCTEXT("InvalidParent_InvalidParentStorageType", "A dynamic collection cannot contain child collections");
				}
				return FCollectionContainerCache::ERecursiveWorkerFlowControl::Stop;
			}
		}

		return FCollectionContainerCache::ERecursiveWorkerFlowControl::Continue;
	};

	CollectionCache->RecursionHelper_DoWork(InGuard, FCollectionNameType(ParentCollectionName, ParentShareType), ECollectionRecursionFlags::SelfAndParents, IsValidParentCollectionWorker);

	return bValidParent;
}

bool FCollectionContainer::UpdateCaches(ECollectionCacheFlags ToUpdate)
{
	FCollectionScopeLock_RW Guard(Lock);

	return CollectionCache->UpdateCaches(Guard, ToUpdate);
}

void FCollectionContainer::HandleFixupRedirectors(ICollectionRedirectorFollower& InRedirectorFollower)
{
	TArray<FCollectionNameType> UpdatedCollections;
	TArray<FSoftObjectPath> AddedObjects;
	TArray<FSoftObjectPath> RemovedObjects;
	{
		FCollectionScopeLock_Write Guard(Lock);

		verifyf(CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects), TEXT("UpdateCaches must be called within a write lock to guarantee subsequent usage to function as expected"));

		const double LoadStartTime = FPlatformTime::Seconds();

		TArray<TPair<FSoftObjectPath, FSoftObjectPath>> ObjectsToRename;

		// Build up the list of redirected object into rename pairs
		{
			const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);
			for (const TPair<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjectInfo : CachedObjects)
			{
				FSoftObjectPath NewObjectPath;
				if (InRedirectorFollower.FixupObject(CachedObjectInfo.Key, NewObjectPath))
				{
					ObjectsToRename.Emplace(CachedObjectInfo.Key, NewObjectPath);
				}
			}
		}

		AddedObjects.Reserve(ObjectsToRename.Num());
		RemovedObjects.Reserve(ObjectsToRename.Num());

		// Handle the rename for each redirected object
		for (const TPair<FSoftObjectPath, FSoftObjectPath>& ObjectToRename : ObjectsToRename)
		{
			AddedObjects.Add(ObjectToRename.Value);
			RemovedObjects.Add(ObjectToRename.Key);

			ReplaceObjectInCollections(Guard, ObjectToRename.Key, ObjectToRename.Value, UpdatedCollections);
		}

		UE_LOGF(LogCollectionManager, Log, "Fixed up redirectors for %d collections in %0.6f seconds (updated %d objects)", AvailableCollections.Num(), FPlatformTime::Seconds() - LoadStartTime, ObjectsToRename.Num());

		for (const TPair<FSoftObjectPath, FSoftObjectPath>& ObjectToRename : ObjectsToRename)
		{
			UE_LOGF(LogCollectionManager, Verbose, "\tRedirected '%ls' to '%ls'", *ObjectToRename.Key.ToString(), *ObjectToRename.Value.ToString());
		}
		if (UpdatedCollections.Num() > 0)
		{
			CollectionCache->HandleCollectionChanged(Guard);
		}
	}

	// Notify every collection that changed, outside of the lock 
	for (const FCollectionNameType& UpdatedCollection : UpdatedCollections)
	{
		AssetsRemovedFromCollectionDelegate.Broadcast(*this, UpdatedCollection, RemovedObjects);
		AssetsAddedToCollectionDelegate.Broadcast(*this, UpdatedCollection, AddedObjects);
	}
}

bool FCollectionContainer::HandleRedirectorsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths, FText* OutError)
{
	bool bSavedAllCollections = true;
	TArray<FCollectionNameType> UpdatedCollections;
	{ 
		FCollectionScopeLock_Write Guard(Lock);
		TSet<FCollectionNameType> CollectionsToSave;
		FTextBuilder ErrorBuilder;

		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			// We don't have a cache for on-disk objects, so we have to do this the slower way and query each collection in turn
			for (const TPair<FCollectionNameType, TSharedRef<FCollection>>& AvailableCollection : AvailableCollections)
			{
				const FCollectionNameType& CollectionKey = AvailableCollection.Key;
				const TSharedRef<FCollection>& Collection = AvailableCollection.Value;

				if (Collection->IsRedirectorInCollection(ObjectPath) && !Collection->IsAutoGenerated())
				{
					CollectionsToSave.Add(CollectionKey);
				}
			}
		}

		for (const FCollectionNameType& CollectionKey : CollectionsToSave)
		{
			if (TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(CollectionKey))
			{
				const TSharedRef<FCollection>& Collection = *CollectionRefPtr;

				FText SaveError;
				constexpr bool bForceCommitToRevisionControl = false;
				if (InternalSaveCollection(Guard, Collection, &SaveError, bForceCommitToRevisionControl))
				{
					CollectionFileCaches[CollectionKey.Type]->IgnoreFileModification(Collection->GetSourceFilename());

					UpdatedCollections.Add(CollectionKey);
				}
				else
				{
					UE_LOGF(LogCollectionManager, Error, "Error saving collection on redirector deletion: %ls", *SaveError.ToString());
					ErrorBuilder.AppendLine(SaveError);
					bSavedAllCollections = false;
				}
			}
		}

		if (OutError)
		{
			*OutError = ErrorBuilder.ToText();
		}
	}

	// Notify every collection that changed, outside of the lock 
	for (const FCollectionNameType& UpdatedCollection : UpdatedCollections)
	{
		AssetsRemovedFromCollectionDelegate.Broadcast(*this, UpdatedCollection, ObjectPaths);
	}

	return bSavedAllCollections;
}

void FCollectionContainer::HandleObjectRenamed(const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath)
{
	TArray<FCollectionNameType> UpdatedCollections;
	TArray<FSoftObjectPath> AddedObjects;
	TArray<FSoftObjectPath> RemovedObjects;
	{
		FCollectionScopeLock_Write Guard(Lock);

		ReplaceObjectInCollections(Guard, OldObjectPath, NewObjectPath, UpdatedCollections);

		AddedObjects.Add(NewObjectPath);
		RemovedObjects.Add(OldObjectPath);

		if (UpdatedCollections.Num() > 0)
		{
			CollectionCache->HandleCollectionChanged(Guard);
		}
	}	

	// Notify every collection that changed, outside the lock
	for (const FCollectionNameType& UpdatedCollection : UpdatedCollections)
	{
		AssetsRemovedFromCollectionDelegate.Broadcast(*this, UpdatedCollection, RemovedObjects);
		AssetsAddedToCollectionDelegate.Broadcast(*this, UpdatedCollection, AddedObjects);
	}
}

void FCollectionContainer::HandleObjectsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths)
{
	TArray<FCollectionNameType> UpdatedCollections;
	{
		FCollectionScopeLock_Write Guard(Lock);

		verifyf(CollectionCache->UpdateCaches(Guard, ECollectionCacheFlags::Objects), TEXT("UpdateCaches must be called within a write lock to guarantee subsequent usage to function as expected"));

		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			RemoveObjectFromCollections(Guard, ObjectPath, UpdatedCollections);
		}

		if (UpdatedCollections.Num() > 0)
		{
			CollectionCache->HandleCollectionChanged(Guard);
		}
	}

	// Notify every collection that changed, outside the lock 
	for (const FCollectionNameType& UpdatedCollection : UpdatedCollections)
	{
		AssetsRemovedFromCollectionDelegate.Broadcast(*this, UpdatedCollection, ObjectPaths);
	}
}

void FCollectionContainer::OnRemovedFromCollectionManager()
{
	FCollectionScopeLock_Write Guard(Lock);

	CollectionManager = nullptr;
}

void FCollectionContainer::TickFileCache()
{
	enum class ECollectionFileAction : uint8
	{
		None,
		AddCollection,
		MergeCollection,
		RemoveCollection,
	};

	// Cached events to fire when we release the lock 
	TArray<TTuple<ECollectionFileAction, FCollectionNameType>> Events;
	{
		// Acquire write lock immediately so we don't need to deal with state change during promotion
		FCollectionScopeLock_Write Guard(Lock);

		// Process changes that have happened outside of the collection container
		for (int32 CacheIdx = 0; CacheIdx < ECollectionShareType::CST_All; ++CacheIdx)
		{
			const ECollectionShareType::Type ShareType = ECollectionShareType::Type(CacheIdx);

			TSharedPtr<DirectoryWatcher::FFileCache>& FileCache = CollectionFileCaches[CacheIdx];
			if (!FileCache.IsValid())
			{
				continue;
			}

			FileCache->Tick();

			const TArray<DirectoryWatcher::FUpdateCacheTransaction> FileCacheChanges = FileCache->GetOutstandingChanges();
			for (const DirectoryWatcher::FUpdateCacheTransaction& FileCacheChange : FileCacheChanges)
			{
				const FString CollectionFilename = FileCacheChange.Filename.Get();
				if (FPaths::GetExtension(CollectionFilename) != CollectionExtension)
				{
					continue;
				}

				const FName CollectionName = *FPaths::GetBaseFilename(CollectionFilename);

				ECollectionFileAction CollectionFileAction = ECollectionFileAction::None;
				switch (FileCacheChange.Action)
				{
				case DirectoryWatcher::EFileAction::Added:
				case DirectoryWatcher::EFileAction::Modified:
					// File was added or modified, but does this collection already exist?
					CollectionFileAction = (AvailableCollections.Contains(FCollectionNameType(CollectionName, ShareType))) 
						? ECollectionFileAction::MergeCollection 
						: ECollectionFileAction::AddCollection;
					break;

				case DirectoryWatcher::EFileAction::Removed:
					// File was removed, but does this collection actually exist?
					CollectionFileAction = (AvailableCollections.Contains(FCollectionNameType(CollectionName, ShareType))) 
						? ECollectionFileAction::RemoveCollection 
						: ECollectionFileAction::None;
					break;

				default:
					break;
				}

				switch (CollectionFileAction)
				{
				case ECollectionFileAction::AddCollection:
					{
						const bool bUseSCC = ShouldUseSCC(ShareType);

						FText LoadErrorText;
						TSharedRef<FCollection> NewCollection = MakeShareable(new FCollection(GetCollectionFilename(CollectionName, ShareType), bUseSCC, ECollectionStorageMode::Static));
						if (NewCollection->Load(LoadErrorText))
						{
							if (AddCollection(Guard, NewCollection, ShareType))
							{
								Events.Emplace(CollectionFileAction, FCollectionNameType(CollectionName, ShareType));
							}
						}
						else
						{
							UE_LOGF(LogCollectionManager, Warning, "%ls", *LoadErrorText.ToString());
						}
					}
					break;

				case ECollectionFileAction::MergeCollection:
					{
						TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(FCollectionNameType(CollectionName, ShareType));
						check(CollectionRefPtr); // We tested AvailableCollections.Contains(...) above, so this shouldn't fail

						FText LoadErrorText;
						FCollection TempCollection(GetCollectionFilename(CollectionName, ShareType), /*bUseSCC*/false, ECollectionStorageMode::Static);
						if (TempCollection.Load(LoadErrorText))
						{
							if ((*CollectionRefPtr)->Merge(TempCollection))
							{
								Events.Emplace(CollectionFileAction, FCollectionNameType(CollectionName, ShareType));
							}
						}
						else
						{
							UE_LOGF(LogCollectionManager, Warning, "%ls", *LoadErrorText.ToString());
						}
					}
					break;

				case ECollectionFileAction::RemoveCollection:
					{
						TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(FCollectionNameType(CollectionName, ShareType));
						check(CollectionRefPtr); // We tested AvailableCollections.Contains(...) above, so this shouldn't fail

						RemoveCollection(Guard, *CollectionRefPtr, ShareType);
						Events.Emplace(CollectionFileAction, FCollectionNameType(CollectionName, ShareType));
					}
					break;

				default:
					break;
				}
			}
		}

		if (Events.Num() > 0)
		{
			CollectionCache->HandleCollectionChanged(Guard);
		}	
	}

	// Broadcast events outside the lock
	for (const TTuple<ECollectionFileAction, FCollectionNameType>& Event : Events)
	{
		switch(Event.Key)
		{
		case ECollectionFileAction::AddCollection:
			CollectionCreatedEvent.Broadcast(*this, Event.Value);
			break;
		case ECollectionFileAction::MergeCollection:
			CollectionUpdatedEvent.Broadcast(*this, Event.Value);
			break;
		case ECollectionFileAction::RemoveCollection:
			CollectionDestroyedEvent.Broadcast(*this, Event.Value);
			break;
		}
	}
}

void FCollectionContainer::LoadCollections()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionContainer::LoadCollections)

	const double LoadStartTime = FPlatformTime::Seconds();
	const int32 PrevNumCollections = AvailableCollections.Num();
	LLM_SCOPE_BYNAME(TEXT("CollectionManager"));

	// This function should only be called during construction, don't acquire a lock here, acquire it for each individual add operation
	ParallelFor(
		TEXT("LoadCollections.PF"),
		ECollectionShareType::CST_All,1,
		[this](int32 CacheIdx)
		{
			const ECollectionShareType::Type ShareType = ECollectionShareType::Type(CacheIdx);
			const bool bUseSCC = ShouldUseSCC(ShareType);
			const FString& CollectionFolder = CollectionSource->GetCollectionFolder(static_cast<ECollectionShareType::Type>(CacheIdx));
			TStringBuilder<256> WildCard{InPlace, CollectionFolder, TEXTVIEW("/*."), CollectionExtension};

			TArray<FString> Filenames;
			IFileManager::Get().FindFiles(Filenames, *WildCard, true, false);

			ParallelFor(
				TEXT("LoadCollections.PF"),
				Filenames.Num(),1,
				[this, &Filenames, &CollectionFolder, bUseSCC, ShareType](int32 FilenameIdx)
				{
					const FString& BaseFilename = Filenames[FilenameIdx];
					const FString Filename = CollectionFolder / BaseFilename;

					FText LoadErrorText;
					TSharedRef<FCollection> NewCollection = MakeShareable(new FCollection(Filename, bUseSCC, ECollectionStorageMode::Static));
					if (NewCollection->Load(LoadErrorText))
					{
						FCollectionScopeLock_Write Guard(Lock);
						AddCollection(Guard, NewCollection, ShareType);
					}
					else
					{
						UE_LOGF(LogCollectionManager, Warning, "%ls", *LoadErrorText.ToString());
					}
				},
				EParallelForFlags::Unbalanced
			);
		},
		EParallelForFlags::Unbalanced
	);

	// AddCollection is assumed to be adding an empty collection, so also notify that collection cache that the collection has "changed" since loaded collections may not always be empty
	FCollectionScopeLock_Write Guard(Lock);
	CollectionCache->HandleCollectionChanged(Guard);

	UE_LOGF(LogCollectionManager, Log, "Loaded %d collections in %0.6f seconds", AvailableCollections.Num() - PrevNumCollections, FPlatformTime::Seconds() - LoadStartTime);
}

bool FCollectionContainer::ShouldUseSCC(ECollectionShareType::Type ShareType) const
{
	return ShareType != ECollectionShareType::CST_Local && ShareType != ECollectionShareType::CST_System;
}

FString FCollectionContainer::GetCollectionFilename(const FName& InCollectionName, const ECollectionShareType::Type InCollectionShareType) const
{
	FString CollectionFilename = CollectionSource->GetCollectionFolder(InCollectionShareType) / InCollectionName.ToString() + TEXT(".") + CollectionExtension;
	FPaths::NormalizeFilename(CollectionFilename);
	return CollectionFilename;
}

uint8 FCollectionContainer::GetReadOnlyMask(ECollectionShareType::Type ShareType)
{
	check(ShareType <= ECollectionShareType::CST_All);

	uint8 Mask = 0;

	if (ShareType == ECollectionShareType::CST_All)
	{
		for (std::underlying_type_t<ECollectionShareType::Type> Value = 0; Value < ECollectionShareType::CST_All; ++Value)
		{
			Mask |= static_cast<uint8>(1 << Value);
		}
	}
	else
	{
		Mask = static_cast<uint8>(1 << ShareType);
	}

	return Mask;
}

bool FCollectionContainer::IsReadOnly(FCollectionScopeLock& InGuard, ECollectionShareType::Type ShareType) const
{
	const uint8 ReadOnlyMask = GetReadOnlyMask(ShareType);
	return (ReadOnlyFlags & ReadOnlyMask) == ReadOnlyMask;
}

bool FCollectionContainer::ValidateWritable(FCollectionScopeLock& InGuard, ECollectionShareType::Type ShareType, FText* OutError) const
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_Internal", "There was an internal error.");
		}
		return false;
	}

	if (IsReadOnly(InGuard, ShareType))
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_ReadOnly", "The collection container is read-only.");
		}
		return false;
	}

	if (CollectionManager == nullptr)
	{
		if (OutError)
		{
			*OutError = LOCTEXT("Error_HasBeenRemoved", "The collection container has been removed.");
		}
		return false;
	}

	return true;
}

bool FCollectionContainer::AddCollection(FCollectionScopeLock_Write& Guard, const TSharedRef<FCollection>& CollectionRef, ECollectionShareType::Type ShareType)
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		return false;
	}

	const FCollectionNameType CollectionKey(CollectionRef->GetCollectionName(), ShareType);
	if (AvailableCollections.Contains(CollectionKey))
	{
		UE_LOGF(LogCollectionManager, Warning, "Failed to add collection '%ls' because it already exists.", *CollectionRef->GetCollectionName().ToString());
		return false;
	}

	AvailableCollections.Add(CollectionKey, CollectionRef);
	CollectionCache->HandleCollectionAdded(Guard);
	return true;
}

bool FCollectionContainer::RemoveCollection(FCollectionScopeLock_Write& Guard, const TSharedRef<FCollection>& CollectionRef, ECollectionShareType::Type ShareType)
{
	if (!ensure(ShareType < ECollectionShareType::CST_All))
	{
		// Bad share type
		return false;
	}

	const FCollectionNameType CollectionKey(CollectionRef->GetCollectionName(), ShareType);
	if (AvailableCollections.Remove(CollectionKey) > 0)
	{
		CollectionCache->HandleCollectionRemoved(Guard);
		return true;
	}

	return false;
}

void FCollectionContainer::RemoveObjectFromCollections(FCollectionScopeLock_Write& Guard, const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutUpdatedCollections)
{
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(Guard);

	const TArray<FObjectCollectionInfo>* ObjectCollectionInfosPtr = CachedObjects.Find(ObjectPath);
	if (!ObjectCollectionInfosPtr)
	{
		return;
	}

	// Remove this object reference from all collections that use it
	for (const FObjectCollectionInfo& ObjectCollectionInfo : *ObjectCollectionInfosPtr)
	{
		if ((ObjectCollectionInfo.Reason & ECollectionRecursionFlags::Self) != 0)
		{
			// The object is contained directly within this collection (rather than coming from a parent or child collection), so remove the object reference
			const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(ObjectCollectionInfo.CollectionKey);
			if (CollectionRefPtr)
			{
				OutUpdatedCollections.AddUnique(ObjectCollectionInfo.CollectionKey);

				(*CollectionRefPtr)->RemoveObjectFromCollection(ObjectPath);
			}
		}
	}
}

void FCollectionContainer::ReplaceObjectInCollections(FCollectionScopeLock_Write& InGuard, const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath, TArray<FCollectionNameType>& OutUpdatedCollections)
{
	verifyf(CollectionCache->UpdateCaches(InGuard, ECollectionCacheFlags::Objects), TEXT("UpdateCaches must be called within a write lock to guarantee subsequent usage to function as expected"));
	const TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>>& CachedObjects = CollectionCache->GetCachedObjects(InGuard);

	const TArray<FObjectCollectionInfo>* OldObjectCollectionInfosPtr = CachedObjects.Find(OldObjectPath);
	if (!OldObjectCollectionInfosPtr)
	{
		return;
	}

	// Replace this object reference in all collections that use it
	for (const FObjectCollectionInfo& OldObjectCollectionInfo : *OldObjectCollectionInfosPtr)
	{
		if ((OldObjectCollectionInfo.Reason & ECollectionRecursionFlags::Self) != 0)
		{
			// The old object is contained directly within this collection (rather than coming from a parent or child collection), so update the object reference
			const TSharedRef<FCollection>* const CollectionRefPtr = AvailableCollections.Find(OldObjectCollectionInfo.CollectionKey);
			if (CollectionRefPtr)
			{
				OutUpdatedCollections.AddUnique(OldObjectCollectionInfo.CollectionKey);

				(*CollectionRefPtr)->RemoveObjectFromCollection(OldObjectPath);
				(*CollectionRefPtr)->AddObjectToCollection(NewObjectPath);
			}
		}
	}
}

bool FCollectionContainer::InternalSaveCollection(FCollectionScopeLock_Write& InGuard, const TSharedRef<FCollection>& CollectionRef, FText* OutError, bool bForceCommitToRevisionControl)
{
	if (!ensure(CollectionManager != nullptr))
	{
		return false;
	}

	TArray<FText> AdditionalChangelistText;

	// Give game specific editors a chance to add lines - do this under the lock because we don't expect re-entrancy
	CollectionManager->OnAddToCollectionCheckinDescriptionEvent().Broadcast(CollectionRef->GetCollectionName(), AdditionalChangelistText);

	// Give the source a chance to add lines
	AdditionalChangelistText.Append(CollectionSource->GetSourceControlCheckInDescription(CollectionRef->GetCollectionName()));

	// Save the collection
	FText UnusedError;
	return CollectionRef->Save(AdditionalChangelistText, OutError ? *OutError : UnusedError, bForceCommitToRevisionControl);
}

#undef LOCTEXT_NAMESPACE
