// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Collection.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "Templates/PimplPtr.h"

namespace DirectoryWatcher
{
	class FFileCache;
}
enum class ECollectionCacheFlags;
class FCollectionContainerCache;
class FCollectionManager;

UE_INTERNAL class FCollectionRecursiveRwLock
{
	struct FThreadLockDepths
	{
		FThreadLockDepths(void* TlsSlotValue);

		void* GetTlsSlotValue();

		using HalfUPtrInt = uint32;

		HalfUPtrInt ThreadReadDepth = 0;
		HalfUPtrInt ThreadWriteDepth = 0;
	};

public:
	FCollectionRecursiveRwLock();
	~FCollectionRecursiveRwLock();

	void ReadLock();
	void WriteLock();

	void ReadUnlock();
	void WriteUnlock();

	// Promote the lock from read to write, possibly being interrupted by another writer in between.
	// Returns false if the lock cannot be promoted, which can happen if the thread already holds a read
	// lock then reenters and tries to promote to a write lock.
	[[nodiscard]] bool PromoteInterruptible();

private:
	FRWLock RwLock;
	uint32 TlsSlot;
};

// Objects wrapping locks to read, write, or begin-reading-then-write (for cache updates) internal state. 
// Used as internal function parameters to show what lock type must be held to perform the operation and prevent 
// recursive lock acquisition
// Functions taking Lock/Lock_Read need to be able to read data but not update caches
class FCollectionScopeLock;
class FCollectionScopeLock_Read;
// Functions taking Lock_RW may need to promote the lock to a write state to update caches
class FCollectionScopeLock_RW;
// Functions taking Lock_Write have exclusive access and can update collections as well as update caches
class FCollectionScopeLock_Write;

class FCollectionContainer final : public ICollectionContainer
{
public:
	FCollectionContainer(FCollectionManager& InCollectionManager, const TSharedRef<ICollectionSource>& InCollectionSource);
	~FCollectionContainer();

	virtual const TSharedRef<ICollectionSource>& GetCollectionSource() const override
	{
		return CollectionSource;
	}

	virtual bool IsReadOnly(ECollectionShareType::Type ShareType) const override;
	virtual void SetReadOnly(ECollectionShareType::Type ShareType, bool bReadOnly) override;
	virtual bool IsHidden() const override;
	virtual void SetHidden(bool bHidden) override;
	virtual bool HasCollections() const override;
	virtual void GetCollections(TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const override;
	virtual void GetRootCollections(TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const override;
	virtual void GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const override;
	virtual TOptional<FCollectionNameType> GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const override;
	virtual bool CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const override;
	virtual bool GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& AssetPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual bool GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual bool GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual FString GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, bool bFullPaths = true) const override;
	virtual FString MakeCollectionPath(FName CollectionName, ECollectionShareType::Type ShareType) const override;
	virtual void CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const override;
	virtual bool IsValidCollectionName(const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) const override;
	virtual bool CreateCollection(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode, FText* OutError = nullptr) override;
	virtual bool RenameCollection(FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName, ECollectionShareType::Type NewShareType, FText* OutError = nullptr) override;
	virtual bool ReparentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) override;
	virtual bool DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) override;
	virtual bool AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumAdded = nullptr, FText* OutError = nullptr) override;
	virtual bool RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) override;
	virtual bool RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumRemoved = nullptr, FText* OutError = nullptr) override;
	virtual bool SetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText, FText* OutError = nullptr) override;
	virtual bool GetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText, FText* OutError = nullptr) const override;
	virtual bool TestDynamicQuery(FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext, bool& OutResult, FText* OutError = nullptr) const override;
	virtual bool EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool GetCollectionStatusInfo(FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo, FText* OutError = nullptr) const override;
	virtual bool HasCollectionColors(TArray<FLinearColor>* OutColors = nullptr) const override;
	virtual bool GetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor, FText* OutError = nullptr) const override;
	virtual bool SetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor, FText* OutError = nullptr) override;
	virtual bool GetCollectionStorageMode(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode, FText* OutError = nullptr) const override;
	virtual bool IsObjectInCollection(const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, FText* OutError = nullptr) const override;
	virtual bool IsValidParentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) const override;

	/*
	* Returns false if the collection container's internal lock cannot be promoted preventing the cache from updating,
	* which can happen if the thread already holds a read lock then reenters and tries to promote to a write lock.
	*/
	bool UpdateCaches(ECollectionCacheFlags ToUpdate);

	void HandleFixupRedirectors(ICollectionRedirectorFollower& InRedirectorFollower);
	bool HandleRedirectorsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths, FText* OutError);
	void HandleObjectRenamed(const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath);
	void HandleObjectsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths);

	/** Event for when the collection container's hidden state changes */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FIsHiddenChangedEvent, FIsHiddenChangedEvent);
	virtual FIsHiddenChangedEvent& OnIsHiddenChanged() override { return IsHiddenChangedEvent; }

	/** Event for when collections are created */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FCollectionCreatedEvent, FCollectionCreatedEvent);
	virtual FCollectionCreatedEvent& OnCollectionCreated() override { return CollectionCreatedEvent; }

	/** Event for when collections are destroyed */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FCollectionDestroyedEvent, FCollectionDestroyedEvent);
	virtual FCollectionDestroyedEvent& OnCollectionDestroyed() override { return CollectionDestroyedEvent; }

	/** Event for when assets are added to a collection */
	virtual FOnAssetsAddedToCollection& OnAssetsAddedToCollection() override { return AssetsAddedToCollectionDelegate; }

	/** Event for when assets are removed from a collection */
	virtual FOnAssetsRemovedFromCollection& OnAssetsRemovedFromCollection() override { return AssetsRemovedFromCollectionDelegate; }

	/** Event for when collections are renamed */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FCollectionRenamedEvent, FCollectionRenamedEvent);
	virtual FCollectionRenamedEvent& OnCollectionRenamed() override { return CollectionRenamedEvent; }

	/** Event for when collections are re-parented */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FCollectionReparentedEvent, FCollectionReparentedEvent);
	virtual FCollectionReparentedEvent& OnCollectionReparented() override { return CollectionReparentedEvent; }

	/** Event for when collections is updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	DECLARE_DERIVED_EVENT(FCollectionContainer, ICollectionContainer::FCollectionUpdatedEvent, FCollectionUpdatedEvent);
	virtual FCollectionUpdatedEvent& OnCollectionUpdated() override { return CollectionUpdatedEvent; }

	void OnRemovedFromCollectionManager();

	/** Tick this collection container so it can process any file cache events */
	void TickFileCache();

private:
	/** Loads all collection files from disk. Must only be called from constructor as it does not lock for the full duration. */
	void LoadCollections();

	/** Returns true if the specified share type requires source control */
	bool ShouldUseSCC(ECollectionShareType::Type ShareType) const;

	/** Given a collection name and share type, work out the full filename for the collection to use on disk */
	FString GetCollectionFilename(const FName& InCollectionName, const ECollectionShareType::Type InCollectionShareType) const;

	/** Returns the read-only bit mask for the specified share type. */
	static uint8 GetReadOnlyMask(ECollectionShareType::Type ShareType);

	bool IsReadOnly(FCollectionScopeLock& InGuard, ECollectionShareType::Type ShareType) const;

	/** Returns whether the collection container is in a valid state for writing */
	bool ValidateWritable(FCollectionScopeLock& InGuard, ECollectionShareType::Type ShareType, FText* OutError) const;

	/** Adds a collection to the lookup maps */
	bool AddCollection(FCollectionScopeLock_Write& InGuard, const TSharedRef<FCollection>& CollectionRef, ECollectionShareType::Type ShareType);

	/** Removes a collection from the lookup maps */
	bool RemoveCollection(FCollectionScopeLock_Write& InGuard, const TSharedRef<FCollection>& CollectionRef, ECollectionShareType::Type ShareType);

	/** Removes an object from any collections that contain it */
	void RemoveObjectFromCollections(FCollectionScopeLock_Write& InGuard, const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutUpdatedCollections);

	/** Replaces an object with another in any collections that contain it */
	void ReplaceObjectInCollections(
		FCollectionScopeLock_Write& InGuard, const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath,
		TArray<FCollectionNameType>& OutUpdatedCollections);

	/** Internal common functionality for saving a collection
	 * bForceCommitToRevisionControl - If the collection's storage mode will save it to source control, then
	 * bForceCommitToRevisionControl will ensure that it is committed after save.  If this is false, then the collection
	 * will be left as a modified file which can be advantageous for slow source control servers.
	 */
	bool InternalSaveCollection(FCollectionScopeLock_Write&, const TSharedRef<FCollection>& CollectionRef, FText* OutError, bool bForceCommitToRevisionControl);

	/* 
	 * Internal version of IsValidParentCollection to avoid taking lock recursively.
	 * Cache must be updated for recursion before calling.
	 */
	bool IsValidParentCollection_Locked(FCollectionScopeLock& InGuard, FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) const;

	/** 
	 * Check if the given collection exists.
	 * Using the public API function risks acquiring the lock recursively.
	 */
	bool CollectionExists_Locked(FCollectionScopeLock& InGuard, FName CollectionName, ECollectionShareType::Type ShareType) const;

private:
	/** The extension used for collection files */
	static FStringView CollectionExtension;

	/** Required for updating caches as well as write operations to collections */
	mutable FCollectionRecursiveRwLock Lock;

	/** The collection managed that is managing this collection container.
	  * Null if this collection container has been removed from the collection manager.
	  */
	FCollectionManager* CollectionManager;

	/** The folders that contain collections */
	TSharedRef<ICollectionSource> CollectionSource;

	/** Bit representation of the read-only state of each share type */
	static_assert(sizeof(uint8) * 8 >= ECollectionShareType::CST_All, "ReadOnlyFlags is not large enough for all share types.");
	uint8 ReadOnlyFlags;

	/** True if the collection container is hidden in the Editor's UI */
	bool bIsHidden;

	/** Array of file cache instances that are watching for the collection files changing on disk */
	TSharedPtr<DirectoryWatcher::FFileCache> CollectionFileCaches[ECollectionShareType::CST_All];

	/** A map of collection names to FCollection objects */
	TMap<FCollectionNameType, TSharedRef<FCollection>> AvailableCollections;

	/** Cache of collection hierarchy, identity, etc */
	TPimplPtr<FCollectionContainerCache> CollectionCache;

	/** Event for when the collection container's hidden state changes */
	FIsHiddenChangedEvent IsHiddenChangedEvent;

	/** Event for when assets are added to a collection */
	FOnAssetsAddedToCollection AssetsAddedToCollectionDelegate;

	/** Event for when assets are removed from a collection */
	FOnAssetsRemovedFromCollection AssetsRemovedFromCollectionDelegate;

	/** Event for when collections are renamed */
	FCollectionRenamedEvent CollectionRenamedEvent;

	/** Event for when collections are re-parented */
	FCollectionReparentedEvent CollectionReparentedEvent;

	/** Event for when collections are updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	FCollectionUpdatedEvent CollectionUpdatedEvent;

	/** Event for when collections are created */
	FCollectionCreatedEvent CollectionCreatedEvent;

	/** Event for when collections are destroyed */
	FCollectionDestroyedEvent CollectionDestroyedEvent;
};
