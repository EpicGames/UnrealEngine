// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionManager.h"

#include "Algo/Transform.h"
#include "Containers/Ticker.h"
#include "CollectionContainer.h"
#include "CollectionManagerLog.h"
#include "CollectionManagerModule.h"
#include "Misc/CommandLine.h"
#include "ProjectCollectionSource.h"
#include "SourceControlPreferences.h"
#include "Tasks/Task.h"

FCollectionManager::FCollectionManager()
	: ProjectCollectionContainer(MakeShared<FCollectionContainer>(*this, MakeShared<FProjectCollectionSource>()))
{
	bNoFixupRedirectors = FParse::Param(FCommandLine::Get(), TEXT("NoFixupRedirectorsInCollections"));

	CollectionContainers.Emplace(StaticCastSharedRef<FCollectionContainer>(ProjectCollectionContainer));

	ProjectCollectionContainer->OnCollectionCreated().AddRaw(this, &FCollectionManager::CollectionCreated);
	ProjectCollectionContainer->OnCollectionDestroyed().AddRaw(this, &FCollectionManager::CollectionDestroyed);
	ProjectCollectionContainer->OnAssetsAddedToCollection().AddRaw(this, &FCollectionManager::AssetsAddedToCollection);
	ProjectCollectionContainer->OnAssetsRemovedFromCollection().AddRaw(this, &FCollectionManager::AssetsRemovedFromCollection);
	ProjectCollectionContainer->OnCollectionRenamed().AddRaw(this, &FCollectionManager::CollectionRenamed);
	ProjectCollectionContainer->OnCollectionReparented().AddRaw(this, &FCollectionManager::CollectionReparented);
	ProjectCollectionContainer->OnCollectionUpdated().AddRaw(this, &FCollectionManager::CollectionUpdated);

	InitializeCollectionContainer(StaticCastSharedRef<FCollectionContainer>(ProjectCollectionContainer));

	TickFileCacheDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCollectionManager::TickFileCache), 1.0f);
}

FCollectionManager::~FCollectionManager()
{
	FTSTicker::RemoveTicker(TickFileCacheDelegateHandle);

	ProjectCollectionContainer->OnCollectionCreated().RemoveAll(this);
	ProjectCollectionContainer->OnCollectionDestroyed().RemoveAll(this);
	ProjectCollectionContainer->OnAssetsAddedToCollection().RemoveAll(this);
	ProjectCollectionContainer->OnAssetsRemovedFromCollection().RemoveAll(this);
	ProjectCollectionContainer->OnCollectionRenamed().RemoveAll(this);
	ProjectCollectionContainer->OnCollectionReparented().RemoveAll(this);
	ProjectCollectionContainer->OnCollectionUpdated().RemoveAll(this);

	// Any references are no longer valid for writing.
	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->OnRemovedFromCollectionManager();
	}
}

const TSharedRef<ICollectionContainer>& FCollectionManager::GetProjectCollectionContainer() const
{
	return ProjectCollectionContainer;
}

TSharedPtr<ICollectionContainer> FCollectionManager::AddCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource)
{
	check(CollectionSource->GetName() != NAME_None);

	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		// CollectionSource is the key, so make sure there is only one container.
		if (CollectionContainer->GetCollectionSource() == CollectionSource)
		{
			UE_LOGF(LogCollectionManager, Warning, "Collection source '%ls' already added.", *CollectionSource->GetName().ToString());
			return nullptr;
		}

		// Names should also be unique.
		if (CollectionContainer->GetCollectionSource()->GetName() == CollectionSource->GetName())
		{
			UE_LOGF(LogCollectionManager, Warning, "Collection source shares the same name with existing collection container '%ls'.", *CollectionSource->GetName().ToString());
			return nullptr;
		}
	}

	TSharedRef<FCollectionContainer> CollectionContainer = MakeShared<FCollectionContainer>(*this, CollectionSource);

	CollectionContainers.Emplace(CollectionContainer);

	InitializeCollectionContainer(CollectionContainer);

	OnCollectionContainerCreated().Broadcast(CollectionContainer);

	return CollectionContainer;
}

bool FCollectionManager::RemoveCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	// The ProjectCollectionContainer cannot be removed for API backwards compatibility.
	if (ProjectCollectionContainer != CollectionContainer)
	{
		for (auto It = CollectionContainers.CreateIterator(); It; ++It)
		{
			if ((*It).Get() == &CollectionContainer.Get())
			{
				// Any collections are no longer valid for writing.
				(*It)->OnRemovedFromCollectionManager();

				It.RemoveCurrent();

				OnCollectionContainerDestroyed().Broadcast(CollectionContainer);
				return true;
			}
		}
	}

	return false;
}

bool FCollectionManager::HasCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer) const
{
	return CollectionContainers.Contains(CollectionContainer);
}

TSharedPtr<ICollectionContainer> FCollectionManager::FindCollectionContainer(FName CollectionSourceName) const
{
	if (CollectionSourceName.IsNone())
	{
		return nullptr;
	}

	const TSharedPtr<FCollectionContainer>* Match = CollectionContainers.FindByPredicate([&CollectionSourceName](const TSharedPtr<FCollectionContainer>& CollectionContainer)
		{
			return CollectionSourceName == CollectionContainer->GetCollectionSource()->GetName();
		});
	return Match != nullptr ? *Match : nullptr;
}

TSharedPtr<ICollectionContainer> FCollectionManager::FindCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource) const
{
	const TSharedPtr<FCollectionContainer>* Match = CollectionContainers.FindByPredicate([&CollectionSource](const TSharedPtr<FCollectionContainer>& CollectionContainer)
		{
			return CollectionSource == CollectionContainer->GetCollectionSource();
		});
	return Match != nullptr ? *Match : nullptr;
}

void FCollectionManager::GetCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const
{
	OutCollectionContainers.Reset(CollectionContainers.Num());
	Algo::Transform(CollectionContainers, OutCollectionContainers, [](const TSharedPtr<FCollectionContainer>& CollectionContainer)
	{
		return CollectionContainer;
	});
}

void FCollectionManager::GetVisibleCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const
{
	OutCollectionContainers.Reset(CollectionContainers.Num());
	Algo::TransformIf(
		CollectionContainers,
		OutCollectionContainers,
		[](const TSharedPtr<FCollectionContainer>& CollectionContainer)
		{
			return !CollectionContainer->IsHidden();
		},
		[](const TSharedPtr<FCollectionContainer>& CollectionContainer)
		{
			return CollectionContainer;
		});
}

bool FCollectionManager::TryParseCollectionPath(const FString& CollectionPath, TSharedPtr<ICollectionContainer>* OutCollectionContainer, FName* OutCollectionName, ECollectionShareType::Type* OutShareType) const
{
	if (CollectionPath.IsEmpty())
	{
		return false;
	}

	// If just a collection name, assume it is from the project collection container.
	if (CollectionPath[0] != TEXT('/'))
	{
		if (OutCollectionContainer != nullptr)
		{
			*OutCollectionContainer = GetProjectCollectionContainer();
		}

		if (OutCollectionName != nullptr)
		{
			*OutCollectionName = FName(CollectionPath);
		}

		if (OutShareType != nullptr)
		{
			*OutShareType = ECollectionShareType::CST_All;
		}

		return true;
	}

	TArray<FString> CollectionPathParts;
	CollectionPath.ParseIntoArray(CollectionPathParts, TEXT("/"));
	
	// If two parts, expect format /CollectionContainer/CollectionName.
	if (CollectionPathParts.Num() == 2)
	{
		if (OutCollectionContainer)
		{
			*OutCollectionContainer = FindCollectionContainer(FName(CollectionPathParts[0], FNAME_Find));

			if (!OutCollectionContainer->IsValid())
			{
				return false;
			}
		}

		if (OutCollectionName != nullptr)
		{
			*OutCollectionName = FName(CollectionPathParts[1]);
		}

		if (OutShareType != nullptr)
		{
			*OutShareType = ECollectionShareType::CST_All;
		}

		return true;
	}

	// If three parts, expect format /CollectionContainer/ShareType/CollectionName.
	if (CollectionPathParts.Num() == 3)
	{
		if (OutCollectionContainer)
		{
			*OutCollectionContainer = FindCollectionContainer(FName(CollectionPathParts[0], FNAME_Find));

			if (!OutCollectionContainer->IsValid())
			{
				return false;
			}
		}

		if (OutCollectionName != nullptr)
		{
			*OutCollectionName = FName(CollectionPathParts[2]);
		}

		if (OutShareType != nullptr)
		{
			*OutShareType = ECollectionShareType::FromString(*CollectionPathParts[1]);

			if (*OutShareType == ECollectionShareType::CST_All)
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool FCollectionManager::HasCollections() const
{
	return ProjectCollectionContainer->HasCollections();
}

void FCollectionManager::GetCollections(TArray<FCollectionNameType>& OutCollections) const
{
	ProjectCollectionContainer->GetCollections(OutCollections);
}

void FCollectionManager::GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const
{
	ProjectCollectionContainer->GetCollections(CollectionName, OutCollections);
}

void FCollectionManager::GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const
{
	ProjectCollectionContainer->GetCollectionNames(ShareType, CollectionNames);
}

void FCollectionManager::GetRootCollections(TArray<FCollectionNameType>& OutCollections) const
{
	ProjectCollectionContainer->GetRootCollections(OutCollections);
}

void FCollectionManager::GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const
{
	ProjectCollectionContainer->GetRootCollectionNames(ShareType, CollectionNames);
}

void FCollectionManager::GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const
{
	ProjectCollectionContainer->GetChildCollections(CollectionName, ShareType, OutCollections);
}

void FCollectionManager::GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const
{
	ProjectCollectionContainer->GetChildCollectionNames(CollectionName, ShareType, ChildShareType, CollectionNames);
}

TOptional<FCollectionNameType> FCollectionManager::GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const
{
	return ProjectCollectionContainer->GetParentCollection(CollectionName, ShareType);
}

bool FCollectionManager::CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const
{
	return ProjectCollectionContainer->CollectionExists(CollectionName, ShareType);
}

bool FCollectionManager::GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& AssetsPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	return ProjectCollectionContainer->GetAssetsInCollection(CollectionName, ShareType, AssetsPaths, RecursionMode);
}

bool FCollectionManager::GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	return ProjectCollectionContainer->GetClassesInCollection(CollectionName, ShareType, ClassPaths, RecursionMode);
}

bool FCollectionManager::GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode) const
{
	return ProjectCollectionContainer->GetObjectsInCollection(CollectionName, ShareType, ObjectPaths, RecursionMode);
}

void FCollectionManager::GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode) const
{
	ProjectCollectionContainer->GetCollectionsContainingObject(ObjectPath, ShareType, OutCollectionNames, RecursionMode);
}

void FCollectionManager::GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode) const
{
	ProjectCollectionContainer->GetCollectionsContainingObject(ObjectPath, OutCollections, RecursionMode);
}

void FCollectionManager::GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode) const
{
	ProjectCollectionContainer->GetCollectionsContainingObjects(ObjectPaths, OutCollectionsAndMatchedObjects, RecursionMode);
}

FString FCollectionManager::GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode, bool bFullPaths) const
{
	return ProjectCollectionContainer->GetCollectionsStringForObject(ObjectPath, ShareType, RecursionMode, bFullPaths);
}

void FCollectionManager::CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const
{
	ProjectCollectionContainer->CreateUniqueCollectionName(BaseName, ShareType, OutCollectionName);
}

bool FCollectionManager::IsValidCollectionName(const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError) const
{
	return ProjectCollectionContainer->IsValidCollectionName(CollectionName, ShareType, OutError);
}

bool FCollectionManager::CreateCollection(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode, FText* OutError)
{
	return ProjectCollectionContainer->CreateCollection(CollectionName, ShareType, StorageMode, OutError);
}

bool FCollectionManager::RenameCollection(FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName, ECollectionShareType::Type NewShareType, FText* OutError)
{
	return ProjectCollectionContainer->RenameCollection(CurrentCollectionName, CurrentShareType, NewCollectionName, NewShareType, OutError);
}

bool FCollectionManager::ReparentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError)
{
	return ProjectCollectionContainer->ReparentCollection(CollectionName, ShareType, ParentCollectionName, ParentShareType, OutError);
}

bool FCollectionManager::DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	return ProjectCollectionContainer->DestroyCollection(CollectionName, ShareType, OutError);
}

bool FCollectionManager::AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError)
{
	return AddToCollection(CollectionName, ShareType, MakeArrayView(&ObjectPath, 1));
}

bool FCollectionManager::AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumAdded, FText* OutError)
{
	return ProjectCollectionContainer->AddToCollection(CollectionName, ShareType, ObjectPaths, OutNumAdded, OutError);
}

bool FCollectionManager::RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError)
{
	return RemoveFromCollection(CollectionName, ShareType, MakeArrayView(&ObjectPath, 1), nullptr, OutError);
}

bool FCollectionManager::RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumRemoved, FText* OutError)
{
	return ProjectCollectionContainer->RemoveFromCollection(CollectionName, ShareType, ObjectPaths, OutNumRemoved, OutError);
}

bool FCollectionManager::SetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText, FText* OutError)
{
	return ProjectCollectionContainer->SetDynamicQueryText(CollectionName, ShareType, InQueryText, OutError);
}

bool FCollectionManager::GetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText, FText* OutError) const
{
	return ProjectCollectionContainer->GetDynamicQueryText(CollectionName, ShareType, OutQueryText, OutError);
}

bool FCollectionManager::TestDynamicQuery(FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext, bool& OutResult, FText* OutError) const
{
	return ProjectCollectionContainer->TestDynamicQuery(CollectionName, ShareType, InContext, OutResult, OutError);
}

bool FCollectionManager::EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	return ProjectCollectionContainer->EmptyCollection(CollectionName, ShareType, OutError);
}

bool FCollectionManager::SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	return ProjectCollectionContainer->SaveCollection(CollectionName, ShareType, OutError);;
}

bool FCollectionManager::UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError)
{
	return ProjectCollectionContainer->UpdateCollection(CollectionName, ShareType, OutError);
}

bool FCollectionManager::GetCollectionStatusInfo(FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo, FText* OutError) const
{
	return ProjectCollectionContainer->GetCollectionStatusInfo(CollectionName, ShareType, OutStatusInfo, OutError);
}

bool FCollectionManager::HasCollectionColors(TArray<FLinearColor>* OutColors) const
{
	return ProjectCollectionContainer->HasCollectionColors(OutColors);
}

bool FCollectionManager::GetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor, FText* OutError) const
{
	return ProjectCollectionContainer->GetCollectionColor(CollectionName, ShareType, OutColor, OutError);
}

bool FCollectionManager::SetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor, FText* OutError)
{
	return ProjectCollectionContainer->SetCollectionColor(CollectionName, ShareType, NewColor, OutError);
}

bool FCollectionManager::GetCollectionStorageMode(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode, FText* OutError) const
{
	return ProjectCollectionContainer->GetCollectionStorageMode(CollectionName, ShareType, OutStorageMode, OutError);
}

bool FCollectionManager::IsObjectInCollection(const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode, FText* OutError) const
{
	return ProjectCollectionContainer->IsObjectInCollection(ObjectPath, CollectionName, ShareType, RecursionMode, OutError);
}

bool FCollectionManager::IsValidParentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) const
{
	return ProjectCollectionContainer->IsValidParentCollection(CollectionName, ShareType, ParentCollectionName, ParentShareType, OutError);
}

void FCollectionManager::HandleFixupRedirectors(ICollectionRedirectorFollower& InRedirectorFollower)
{
	if (bNoFixupRedirectors)
	{
		return;
	}

	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->HandleFixupRedirectors(InRedirectorFollower);
	}
}

bool FCollectionManager::HandleRedirectorsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths, FText* OutError)
{
	if (ObjectPaths.IsEmpty())
	{
		return true;
	}

	bool bResult = true;
	FTextBuilder ErrorBuilder;
	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		FText Error;
		if (!CollectionContainer->HandleRedirectorsDeleted(ObjectPaths, &Error))
		{
			bResult = false;
			ErrorBuilder.AppendLine(Error);
		}
	}

	if (OutError)
	{
		*OutError = ErrorBuilder.ToText();
	}

	return bResult;
}

bool FCollectionManager::HandleRedirectorDeleted(const FSoftObjectPath& ObjectPath, FText* Error)
{
	return HandleRedirectorsDeleted(MakeArrayView(&ObjectPath, 1), Error);
}

void FCollectionManager::HandleObjectRenamed(const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath)
{
	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->HandleObjectRenamed(OldObjectPath, NewObjectPath);
	}
}

void FCollectionManager::HandleObjectsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths)
{
	check(IsInGameThread());

	if (ObjectPaths.IsEmpty())
	{
		return;
	}

	if (SuppressObjectDeletionRefCount > 0)
	{
		DeferredDeletedObjects.Append(ObjectPaths);
		return;
	}

	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->HandleObjectsDeleted(ObjectPaths);
	}
}

void FCollectionManager::HandleObjectDeleted(const FSoftObjectPath& ObjectPath)
{
	HandleObjectsDeleted(MakeArrayView(&ObjectPath, 1));
}

void FCollectionManager::SuppressObjectDeletionHandling()
{
	check(IsInGameThread());

	SuppressObjectDeletionRefCount++;
}

void FCollectionManager::ResumeObjectDeletionHandling()
{
	check(IsInGameThread());

	int32 PrevRefCount = SuppressObjectDeletionRefCount--;
	ensure(PrevRefCount >= 1);

	if (PrevRefCount == 1 && !DeferredDeletedObjects.IsEmpty())
	{
		for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
		{
			CollectionContainer->HandleObjectsDeleted(DeferredDeletedObjects);
		}

		DeferredDeletedObjects.Empty();
	}
}

void FCollectionManager::InitializeCollectionContainer(const TSharedRef<FCollectionContainer>& CollectionContainer)
{
	// Perform initial caching of collection information ready for user to interact with anything 
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [CollectionContainerWeakPtr = CollectionContainer.ToWeakPtr()]()
		{
			if (TSharedPtr<FCollectionContainer> CollectionContainerSharedPtr = CollectionContainerWeakPtr.Pin())
			{
				CollectionContainerSharedPtr->UpdateCaches(ECollectionCacheFlags::All);
			}
		});
}

bool FCollectionManager::TickFileCache(float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCollectionManager_TickFileCache);

	for (const TSharedPtr<FCollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->TickFileCache();
	}

	return true; // Tick again
}

void FCollectionManager::CollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	OnCollectionCreated().Broadcast(Collection);
}

void FCollectionManager::CollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	OnCollectionDestroyed().Broadcast(Collection);
}

void FCollectionManager::AssetsAddedToCollection(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded)
{
	OnAssetsAddedToCollection().Broadcast(Collection, AssetsAdded);
}

void FCollectionManager::AssetsRemovedFromCollection(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved)
{
	OnAssetsRemovedFromCollection().Broadcast(Collection, AssetsRemoved);
}

void FCollectionManager::CollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection)
{
	OnCollectionRenamed().Broadcast(OriginalCollection, NewCollection);
}

void FCollectionManager::CollectionReparented(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, const TOptional<FCollectionNameType>& OldParent, const TOptional<FCollectionNameType>& NewParent)
{
	OnCollectionReparented().Broadcast(Collection, OldParent, NewParent);
}

void FCollectionManager::CollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	OnCollectionUpdated().Broadcast(Collection);
}
