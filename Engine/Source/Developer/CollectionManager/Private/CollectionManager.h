// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Collection.h"
#include "CollectionManagerTypes.h"
#include "ICollectionManager.h"
#include "Misc/Guid.h"
#include "Templates/PimplPtr.h"

class FCollectionContainer;
class ITextFilterExpressionContext;

/** Collection info for a given object - gives the collection name, as well as the reason this object is considered to be part of this collection */
struct FObjectCollectionInfo
{
	explicit FObjectCollectionInfo(const FCollectionNameType& InCollectionKey)
		: CollectionKey(InCollectionKey)
		, Reason(0)
	{
	}

	FObjectCollectionInfo(const FCollectionNameType& InCollectionKey, const ECollectionRecursionFlags::Flags InReason)
		: CollectionKey(InCollectionKey)
		, Reason(InReason)
	{
	}

	/** The key identifying the collection that contains this object */
	FCollectionNameType CollectionKey;
	/** The reason(s) why this collection contains this object - this can be tested against the recursion mode when getting the collections for an object */
	ECollectionRecursionFlags::Flags Reason;
};

enum class ECollectionCacheFlags
{
	None = 0,
	Names = 1<<0,
	Objects = 1<<1,
	Hierarchy = 1<<2,
	Colors = 1 <<3,

	// Necessary cache updates for calling collection recursion worker
	RecursionWorker = Names | Hierarchy,
	All = Names | Objects | Hierarchy | Colors,
};
ENUM_CLASS_FLAGS(ECollectionCacheFlags);


UE_DEPRECATED(5.5, "These typedefs have been deprecated. Replace them with their concrete types.")
typedef TMap<FCollectionNameType, TSharedRef<FCollection>> FAvailableCollectionsMap;
UE_DEPRECATED(5.5, "These typedefs have been deprecated. Replace them with their concrete types.")
typedef TMap<FGuid, FCollectionNameType> FGuidToCollectionNamesMap;
UE_DEPRECATED(5.5, "These typedefs have been deprecated. Replace them with their concrete types.")
typedef TMap<FSoftObjectPath, TArray<FObjectCollectionInfo>> FCollectionObjectsMap;
UE_DEPRECATED(5.5, "These typedefs have been deprecated. Replace them with their concrete types.")
typedef TMap<FGuid, TArray<FGuid>> FCollectionHierarchyMap;
UE_DEPRECATED(5.5, "These typedefs have been deprecated. Replace them with their concrete types.")
typedef TArray<FLinearColor> FCollectionColorArray;

class FCollectionManager final : public ICollectionManager
{
public:
	FCollectionManager();
	virtual ~FCollectionManager();

	virtual const TSharedRef<ICollectionContainer>& GetProjectCollectionContainer() const override;
	virtual TSharedPtr<ICollectionContainer> AddCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource) override;
	virtual bool RemoveCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer) override;
	virtual bool HasCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer) const override;
	virtual TSharedPtr<ICollectionContainer> FindCollectionContainer(FName CollectionSourceName) const override;
	virtual TSharedPtr<ICollectionContainer> FindCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource) const override;
	virtual void GetCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const override;
	virtual void GetVisibleCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const override;
	virtual bool TryParseCollectionPath(const FString& CollectionPath, TSharedPtr<ICollectionContainer>* OutCollectionContainer, FName* OutCollectionName, ECollectionShareType::Type* OutShareType) const override;

	virtual bool HasCollections() const override;
	virtual void GetCollections(TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const override;
	virtual void GetRootCollections(TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const override;
	virtual void GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const override;
	virtual void GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const override;
	virtual TOptional<FCollectionNameType> GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const override;
	virtual bool CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const override;
	virtual bool GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& AssetPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual bool GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual bool GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual void GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const override;
	virtual FString GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, bool bFullPaths = true) const override;
	virtual void CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const override;
	virtual bool IsValidCollectionName(const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) const override;
	virtual bool CreateCollection(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode, FText* OutError = nullptr) override;
	virtual bool RenameCollection(FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName, ECollectionShareType::Type NewShareType, FText* OutError = nullptr) override;
	virtual bool ReparentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) override;
	virtual bool DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) override;
	virtual bool AddToCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumAdded = nullptr, FText* OutError = nullptr) override;
	virtual bool RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) override;
	virtual bool RemoveFromCollection(FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths, int32* OutNumRemoved = nullptr, FText* OutError = nullptr) override;
	virtual bool SetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText, FText* OutError = nullptr) override;
	virtual bool GetDynamicQueryText(FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText, FText* OutError = nullptr) const override;
	virtual bool TestDynamicQuery(FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext, bool& OutResult, FText* OutError = nullptr) const override;
	virtual bool EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) override;
	virtual bool GetCollectionStatusInfo(FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo, FText* OutError = nullptr) const override;
	virtual bool HasCollectionColors(TArray<FLinearColor>* OutColors = nullptr) const override;
	virtual bool GetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor, FText* OutError = nullptr) const override;
	virtual bool SetCollectionColor(FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor, FText* OutError = nullptr) override;
	virtual bool GetCollectionStorageMode(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode, FText* OutError = nullptr) const override;
	virtual bool IsObjectInCollection(const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, FText* OutError = nullptr) const override;
	virtual bool IsValidParentCollection(FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName, ECollectionShareType::Type ParentShareType, FText* OutError) const override;
	UE_DEPRECATED(5.5, "Deprecated for thread safety reasons.")
	virtual FText GetLastError() const override { return FText::GetEmpty(); }
	virtual void HandleFixupRedirectors(ICollectionRedirectorFollower& InRedirectorFollower) override;
	virtual bool HandleRedirectorDeleted(const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) override;
	virtual bool HandleRedirectorsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths, FText* OutError = nullptr) override;
	virtual void HandleObjectRenamed(const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath) override;
	virtual void HandleObjectDeleted(const FSoftObjectPath& ObjectPath) override;
	virtual void HandleObjectsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths) override;

	virtual void SuppressObjectDeletionHandling() override;
	virtual void ResumeObjectDeletionHandling() override;

	/** Event for when collection containers are created */
	DECLARE_DERIVED_EVENT(FCollectionManager, ICollectionManager::FCollectionContainerCreatedEvent, FCollectionContainerCreatedEvent);
	virtual FCollectionContainerCreatedEvent& OnCollectionContainerCreated() override { return CollectionContainerCreatedEvent; }

	/** Event for when collection containers are destroyed */
	DECLARE_DERIVED_EVENT(FCollectionManager, ICollectionManager::FCollectionContainerDestroyedEvent, FCollectionContainerDestroyedEvent);
	virtual FCollectionContainerDestroyedEvent& OnCollectionContainerDestroyed() override { return CollectionContainerDestroyedEvent; }

	/** Event for when collections are created */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FCollectionCreatedEvent, FCollectionCreatedEvent );
	virtual FCollectionCreatedEvent& OnCollectionCreated() override { return CollectionCreatedEvent; }

	/** Event for when collections are destroyed */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FCollectionDestroyedEvent, FCollectionDestroyedEvent );
	virtual FCollectionDestroyedEvent& OnCollectionDestroyed() override { return CollectionDestroyedEvent; }

	/** Event for when assets are added to a collection */
	virtual FOnAssetsAddedToCollection& OnAssetsAddedToCollection() override { return AssetsAddedToCollectionDelegate; }

	/** Event for when assets are removed from a collection */
	virtual FOnAssetsRemovedFromCollection& OnAssetsRemovedFromCollection() override { return AssetsRemovedFromCollectionDelegate; }

	/** Event for when collections are renamed */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FCollectionRenamedEvent, FCollectionRenamedEvent );
	virtual FCollectionRenamedEvent& OnCollectionRenamed() override { return CollectionRenamedEvent; }

	/** Event for when collections are re-parented */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FCollectionReparentedEvent, FCollectionReparentedEvent );
	virtual FCollectionReparentedEvent& OnCollectionReparented() override { return CollectionReparentedEvent; }

	/** Event for when collections is updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FCollectionUpdatedEvent, FCollectionUpdatedEvent );
	virtual FCollectionUpdatedEvent& OnCollectionUpdated() override { return CollectionUpdatedEvent; }

	/** Event for when collections is updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	DECLARE_DERIVED_EVENT( FCollectionManager, ICollectionManager::FAddToCollectionCheckinDescriptionEvent, FAddToCollectionCheckinDescriptionEvent);
	virtual FAddToCollectionCheckinDescriptionEvent& OnAddToCollectionCheckinDescriptionEvent() override { return AddToCollectionCheckinDescriptionEvent; }

private:
	static void InitializeCollectionContainer(const TSharedRef<class FCollectionContainer>& CollectionContainer);

	/** Tick this collection manager so it can process any file cache events */
	bool TickFileCache(float InDeltaTime);

	void CollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);
	void CollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);
	void AssetsAddedToCollection(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded);
	void AssetsRemovedFromCollection(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved);
	void CollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection);
	void CollectionReparented(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, const TOptional<FCollectionNameType>& OldParent, const TOptional<FCollectionNameType>& NewParent);
	void CollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

private:
	/** Delegate handle for the TickFileCache function */
	FTSTicker::FDelegateHandle TickFileCacheDelegateHandle;

	/** The collection container for the current uproject */
	TSharedRef<ICollectionContainer> ProjectCollectionContainer;

	/** All collection containers */
	TArray<TSharedPtr<FCollectionContainer>> CollectionContainers;

	TArray<FSoftObjectPath> DeferredDeletedObjects;

	/** Event for when collection containers are created */
	FCollectionContainerCreatedEvent CollectionContainerCreatedEvent;

	/** Event for when collection containers are destroyed */
	FCollectionContainerDestroyedEvent CollectionContainerDestroyedEvent;

	/** Event for when assets are added to a collection */
	FOnAssetsAddedToCollection AssetsAddedToCollectionDelegate;

	/** Event for when assets are removed from a collection */
	FOnAssetsRemovedFromCollection AssetsRemovedFromCollectionDelegate;

	/** Event for when collections are renamed */
	FCollectionRenamedEvent CollectionRenamedEvent;

	/** Event for when collections are re-parented */
	FCollectionReparentedEvent CollectionReparentedEvent;

	/** Event for when collections are updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	FCollectionUpdatedEvent CollectionUpdatedEvent;

	/** Event for when collections are created */
	FCollectionCreatedEvent CollectionCreatedEvent;

	/** Event for when collections are destroyed */
	FCollectionDestroyedEvent CollectionDestroyedEvent;

	/** When a collection checkin happens, use this event to add additional text to the changelist description */
	FAddToCollectionCheckinDescriptionEvent AddToCollectionCheckinDescriptionEvent;

	/** Ref count for deferring calls to HandleObjectsDeleted. When the ref count reaches 0 we flush all deferred notifications */
	int32 SuppressObjectDeletionRefCount = 0;

	/** When true, redirectors will not be automatically followed in collections during startup */
	bool bNoFixupRedirectors;
};
