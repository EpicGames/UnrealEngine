// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerLog.h"

#define LOCTEXT_NAMESPACE "CollectionManager"

class FCollectionManagerConsoleCommands
{
public:
	const FCollectionManagerModule& Module;
	
	FAutoConsoleCommand CreateCommand;
	FAutoConsoleCommand DestroyCommand;
	FAutoConsoleCommand AddCommand;
	FAutoConsoleCommand RemoveCommand;

	FCollectionManagerConsoleCommands(const FCollectionManagerModule& InModule)
		: Module(InModule)
	,	CreateCommand(
		TEXT( "CollectionManager.Create" ),
		*LOCTEXT("CommandText_Create", "Creates a collection of the specified name and type").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FCollectionManagerConsoleCommands::Create ) )
	,	DestroyCommand(
		TEXT( "CollectionManager.Destroy" ),
		*LOCTEXT("CommandText_Destroy", "Deletes a collection of the specified name and type").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FCollectionManagerConsoleCommands::Destroy ) )
	,	AddCommand(
		TEXT( "CollectionManager.Add" ),
		*LOCTEXT("CommandText_Add", "Adds the specified object path to the specified collection").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FCollectionManagerConsoleCommands::Add ) )
	,	RemoveCommand(
		TEXT( "CollectionManager.Remove" ),
		*LOCTEXT("CommandText_Remove", "Removes the specified object path from the specified collection").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FCollectionManagerConsoleCommands::Remove ) )
	{}

	void Create(const TArray<FString>& Args)
	{
		if ( Args.Num() < 2 )
		{
			UE_LOGF(LogCollectionManager, Log, "Usage: CollectionManager.Create [CollectionContainer] CollectionName CollectionType [CollectionStorageMode]");
			return;
		}

		// Check if the last arg is a valid storage mode.
		const ECollectionStorageMode::Type InvalidStorageMode = ECollectionStorageMode::Type(-1);
		ECollectionStorageMode::Type StorageMode = InvalidStorageMode;
		if (Args.Num() >= 3)
		{
			const FString& StorageModeStr = Args[Args.Num() - 1];
			StorageMode = ECollectionStorageMode::FromString(*StorageModeStr, InvalidStorageMode);

			// If we have four args, the last one must be the storage mode,
			// otherwise if we have three args it may be that the first one is the collection container instead.
			if (Args.Num() >= 4 && StorageMode == InvalidStorageMode)
			{
				UE_LOGF(LogCollectionManager, Warning, "Invalid collection storage mode: %ls", *StorageModeStr);
				return;
			}
		}

		int32 ArgIndex = 0;
		TSharedPtr<ICollectionContainer> CollectionContainer;
		// If we have four args, or we have three args and the last arg is not a ECollectionStorageMode, then the user should have supplied the collection container.
		if (Args.Num() >= 4 || (Args.Num() == 3 && StorageMode == InvalidStorageMode))
		{
			const FString& CollectionSourceName = Args[ArgIndex++];
			CollectionContainer = Module.Get().FindCollectionContainer(FName(CollectionSourceName, FNAME_Find));
			if (!CollectionContainer)
			{
				UE_LOGF(LogCollectionManager, Warning, "Invalid collection container: %ls", *CollectionSourceName);
				return;
			}
		}
		else
		{
			CollectionContainer = Module.Get().GetProjectCollectionContainer();
		}

		FName CollectionName = FName(*Args[ArgIndex++]);

		FString ShareStr = Args[ArgIndex++];
		ECollectionShareType::Type ShareType;

		if ( ShareStr == TEXT("LOCAL") )
		{
			ShareType = ECollectionShareType::CST_Local;
		}
		else if ( ShareStr == TEXT("PRIVATE") )
		{
			ShareType = ECollectionShareType::CST_Private;
		}
		else if ( ShareStr == TEXT("SHARED") )
		{
			ShareType = ECollectionShareType::CST_Shared;
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Invalid collection share type: %ls", *ShareStr);
			return;
		}

		if (StorageMode == InvalidStorageMode)
		{
			StorageMode = ECollectionStorageMode::Static;
		}

		if (CollectionContainer->CreateCollection(CollectionName, ShareType, StorageMode) )
		{
			UE_LOGF(LogCollectionManager, Log, "Collection created: %ls", *CollectionName.ToString());
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Failed to create collection: %ls", *CollectionName.ToString());
		}
	}

	void Destroy(const TArray<FString>& Args)
	{
		if ( Args.Num() < 2 )
		{
			UE_LOGF(LogCollectionManager, Log, "Usage: CollectionManager.Destroy [CollectionContainer] CollectionName CollectionType");
			return;
		}

		int32 ArgIndex = 0;
		TSharedPtr<ICollectionContainer> CollectionContainer;
		if (Args.Num() >= 3)
		{
			const FString& CollectionSourceName = Args[ArgIndex++];
			CollectionContainer = Module.Get().FindCollectionContainer(FName(CollectionSourceName, FNAME_Find));
			if (!CollectionContainer)
			{
				UE_LOGF(LogCollectionManager, Warning, "Invalid collection container: %ls", *CollectionSourceName);
				return;
			}
		}
		else
		{
			CollectionContainer = Module.Get().GetProjectCollectionContainer();
		}

		FName CollectionName = FName(*Args[ArgIndex++]);
		const FString& ShareStr = Args[ArgIndex++];
		ECollectionShareType::Type Type;

		if ( ShareStr == TEXT("LOCAL") )
		{
			Type = ECollectionShareType::CST_Local;
		}
		else if ( ShareStr == TEXT("PRIVATE") )
		{
			Type = ECollectionShareType::CST_Private;
		}
		else if ( ShareStr == TEXT("SHARED") )
		{
			Type = ECollectionShareType::CST_Shared;
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Invalid collection share type: %ls", *ShareStr);
			return;
		}
		
		if ( CollectionContainer->DestroyCollection(CollectionName, Type) )
		{
			UE_LOGF(LogCollectionManager, Log, "Collection destroyed: %ls", *CollectionName.ToString());
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Failed to destroyed collection: %ls", *CollectionName.ToString());
		}
	}

	void Add(const TArray<FString>& Args)
	{
		if ( Args.Num() < 3 )
		{
			UE_LOGF(LogCollectionManager, Log, "Usage: CollectionManager.Add [CollectionContainer] CollectionName CollectionType ObjectPath");
			return;
		}

		int32 ArgIndex = 0;
		TSharedPtr<ICollectionContainer> CollectionContainer;
		if (Args.Num() >= 4)
		{
			const FString& CollectionSourceName = Args[ArgIndex++];
			CollectionContainer = Module.Get().FindCollectionContainer(FName(CollectionSourceName, FNAME_Find));
			if (!CollectionContainer)
			{
				UE_LOGF(LogCollectionManager, Warning, "Invalid collection container: %ls", *CollectionSourceName);
				return;
			}
		}
		else
		{
			CollectionContainer = Module.Get().GetProjectCollectionContainer();
		}

		FName CollectionName = FName(*Args[ArgIndex++]);
		const FString& ShareStr = Args[ArgIndex++];
		FSoftObjectPath ObjectPath(Args[ArgIndex++]);
		ECollectionShareType::Type Type;

		if ( ShareStr == TEXT("LOCAL") )
		{
			Type = ECollectionShareType::CST_Local;
		}
		else if ( ShareStr == TEXT("PRIVATE") )
		{
			Type = ECollectionShareType::CST_Private;
		}
		else if ( ShareStr == TEXT("SHARED") )
		{
			Type = ECollectionShareType::CST_Shared;
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Invalid collection share type: %ls", *ShareStr);
			return;
		}
		
		if ( CollectionContainer->AddToCollection(CollectionName, Type, ObjectPath) )
		{
			UE_LOGF(LogCollectionManager, Log, "%ls added to collection %ls", *ObjectPath.ToString(), *CollectionName.ToString());
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Failed to add %ls to collection %ls", *ObjectPath.ToString(), *CollectionName.ToString());
		}
	}

	void Remove(const TArray<FString>& Args)
	{
		if ( Args.Num() < 3 )
		{
			UE_LOGF(LogCollectionManager, Log, "Usage: CollectionManager.Remove [CollectionContainer] CollectionName CollectionType ObjectPath");
			return;
		}

		int32 ArgIndex = 0;
		TSharedPtr<ICollectionContainer> CollectionContainer;
		if (Args.Num() >= 4)
		{
			const FString& CollectionSourceName = Args[ArgIndex++];
			CollectionContainer = Module.Get().FindCollectionContainer(FName(CollectionSourceName, FNAME_Find));
			if (!CollectionContainer)
			{
				UE_LOGF(LogCollectionManager, Warning, "Invalid collection container: %ls", *CollectionSourceName);
				return;
			}
		}
		else
		{
			CollectionContainer = Module.Get().GetProjectCollectionContainer();
		}

		FName CollectionName = FName(*Args[ArgIndex++]);
		const FString& ShareStr = Args[ArgIndex++];
		FSoftObjectPath ObjectPath(Args[ArgIndex++]);

		ECollectionShareType::Type Type;

		if ( ShareStr == TEXT("LOCAL") )
		{
			Type = ECollectionShareType::CST_Local;
		}
		else if ( ShareStr == TEXT("PRIVATE") )
		{
			Type = ECollectionShareType::CST_Private;
		}
		else if ( ShareStr == TEXT("SHARED") )
		{
			Type = ECollectionShareType::CST_Shared;
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Invalid collection share type: %ls", *ShareStr);
			return;
		}
		
		if ( CollectionContainer->RemoveFromCollection(CollectionName, Type, ObjectPath) )
		{
			UE_LOGF(LogCollectionManager, Log, "%ls removed from collection %ls", *ObjectPath.ToString(), *CollectionName.ToString());
		}
		else
		{
			UE_LOGF(LogCollectionManager, Warning, "Failed to remove %ls from collection %ls", *ObjectPath.ToString(), *CollectionName.ToString());
		}
	}
};


#undef LOCTEXT_NAMESPACE
