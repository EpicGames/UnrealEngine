// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionManagerModule.h"
#include "CollectionManager.h"
#include "CollectionManagerConsoleCommands.h"

IMPLEMENT_MODULE( FCollectionManagerModule, CollectionManager );
DEFINE_LOG_CATEGORY(LogCollectionManager);

void FCollectionManagerModule::StartupModule()
{
	CollectionManager = new FCollectionManager();
	ConsoleCommands = new FCollectionManagerConsoleCommands(*this);
}

void FCollectionManagerModule::ShutdownModule()
{
	if (CollectionManager != NULL)
	{
		delete CollectionManager;
		CollectionManager = NULL;
	}

	if (ConsoleCommands != NULL)
	{
		delete ConsoleCommands;
		ConsoleCommands = NULL;
	}
}

ICollectionManager& FCollectionManagerModule::Get() const
{
	check(CollectionManager);
	return *CollectionManager;
}
