// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionSettings.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollectionSettings)

#if WITH_EDITOR
FText UCollectionSettings::GetSectionText() const
{
 	return NSLOCTEXT("Collections", "CollectionsSettingsSection", "Collections");
}

FText UCollectionSettings::GetSectionDescription() const
{
	return NSLOCTEXT("Collections", "CollectionsSettingsDescription", "Settings affecting Asset Collections in Editor");
}
#endif
