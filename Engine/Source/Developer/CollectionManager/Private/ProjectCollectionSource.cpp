// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectCollectionSource.h"

#include "Misc/Paths.h"
#include "SourceControlPreferences.h"

#define LOCTEXT_NAMESPACE "CollectionManager"

FProjectCollectionSource::FProjectCollectionSource()
{
	CollectionFolders[ECollectionShareType::CST_Local] = FPaths::ProjectSavedDir() / TEXT("Collections");
	CollectionFolders[ECollectionShareType::CST_Private] = FPaths::GameUserDeveloperDir() / TEXT("Collections");
	CollectionFolders[ECollectionShareType::CST_Shared] = FPaths::ProjectContentDir() / TEXT("Collections");
}

FName FProjectCollectionSource::GetName() const
{
	return NAME_Game;
}

FText FProjectCollectionSource::GetTitle() const
{
	return LOCTEXT("ProjectCollectionSource_Name", "Collections");
}

const FString& FProjectCollectionSource::GetCollectionFolder(const ECollectionShareType::Type InCollectionShareType) const
{
	return CollectionFolders[InCollectionShareType];
}

FString FProjectCollectionSource::GetEditorPerProjectIni() const
{
	return GEditorPerProjectIni;
}

FString FProjectCollectionSource::GetSourceControlStatusHintFilename() const
{
	return FPaths::GetProjectFilePath();
}

TArray<FText> FProjectCollectionSource::GetSourceControlCheckInDescription(FName CollectionName) const
{
	TArray<FString> SettingsLines;

	const USourceControlPreferences* Settings = GetDefault<USourceControlPreferences>();
	if (const FString* SpecificMatch = Settings->SpecificCollectionChangelistTags.Find(CollectionName))
	{
		// Parse input buffer into an array of lines
		SpecificMatch->ParseIntoArrayLines(SettingsLines, /*bCullEmpty=*/ false);
	}
	SettingsLines.Append(Settings->CollectionChangelistTags);

	TArray<FText> CheckInDescription;
	CheckInDescription.Reserve(SettingsLines.Num());
	for (const FString& OneSettingLine : SettingsLines)
	{
		CheckInDescription.Add(FText::FromString(*OneSettingLine));
	}
	return CheckInDescription;
}

#undef LOCTEXT_NAMESPACE
