// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICollectionSource.h"

class FProjectCollectionSource : public ICollectionSource
{
public:
	FProjectCollectionSource();

	FName GetName() const override;

	FText GetTitle() const override;

	const FString& GetCollectionFolder(const ECollectionShareType::Type InCollectionShareType) const override;

	FString GetEditorPerProjectIni() const override;

	FString GetSourceControlStatusHintFilename() const override;

	TArray<FText> GetSourceControlCheckInDescription(FName CollectionName) const override;

private:
	FString CollectionFolders[ECollectionShareType::CST_All];
};
