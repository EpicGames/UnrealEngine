// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class ICollectionManager;

class FCollectionManagerModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	virtual ICollectionManager& Get() const;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FCollectionManagerModule& GetModule()
	{
		static const FName CollectionManagerModuleName("CollectionManager");
		FCollectionManagerModule* Module = FModuleManager::GetModulePtr<FCollectionManagerModule>(CollectionManagerModuleName);
		if (Module == nullptr)
		{
			return FModuleManager::LoadModuleChecked<FCollectionManagerModule>(CollectionManagerModuleName);
		}
		else
		{
			return *Module;
		}
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsModuleAvailable()
	{
		static const FName CollectionManagerModuleName("CollectionManager");
		return FModuleManager::Get().IsModuleLoaded(CollectionManagerModuleName);
	}

private:
	ICollectionManager* CollectionManager;
	class FCollectionManagerConsoleCommands* ConsoleCommands;
};
