// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumRange.h"
#include "UObject/SoftObjectPath.h"
#include "ISourceControlState.h"

class ICollectionContainer;

struct ECollectionStorageMode
{
	enum Type
	{
		/** This collection stores a list of contained objects. */
		Static,
		/** This collection stores a query that can be run against the Content Browser. This type of collection never directly knows what objects it contains. */
		Dynamic,
	};

	static Type FromString(const TCHAR* InString, const Type ReturnIfConversionFails = Static)
	{
		if(FPlatformString::Stricmp(InString, TEXT("Static")) == 0)		return Static;
		if(FPlatformString::Stricmp(InString, TEXT("Dynamic")) == 0)	return Dynamic;
		return ReturnIfConversionFails;
	}

	static const TCHAR* ToString(const Type InType)
	{
		switch(InType)
		{
		case Static:
			return TEXT("Static");
		case Dynamic:
			return TEXT("Dynamic");
		default:
			break;
		}
		return TEXT("");
	}

	static FText ToText(const Type InType)
	{
		switch(InType)
		{
		case Static:
			return NSLOCTEXT("ECollectionStorageMode", "Static", "Static");
		case Dynamic:
			return NSLOCTEXT("ECollectionStorageMode", "Dynamic", "Dynamic");
		default:
			break;
		}
		return FText::GetEmpty();
	}

	static FText GetDescription(const Type InType)
	{
		switch (InType)
		{
		case Static:
			return NSLOCTEXT("ECollectionStorageMode", "Static_Description", "Static. This collection stores a list of contained objects.");
		case Dynamic:
			return NSLOCTEXT("ECollectionStorageMode", "Dynamic_Description", "Dynamic. This collection stores a query that can be run against the Content Browser.");
		default:
			break;
		}
		return FText::GetEmpty();
	}
};

struct ECollectionShareType
{
	enum Type
	{
		CST_System,
		CST_Local,
		CST_Private,
		CST_Shared,

		CST_All
	};

	static Type FromString(const TCHAR* InString, const Type ReturnIfConversionFails = CST_All)
	{
		if(FPlatformString::Stricmp(InString, TEXT("System")) == 0)		return CST_System;
		if(FPlatformString::Stricmp(InString, TEXT("Local")) == 0)		return CST_Local;
		if(FPlatformString::Stricmp(InString, TEXT("Private")) == 0)	return CST_Private;
		if(FPlatformString::Stricmp(InString, TEXT("Shared")) == 0)		return CST_Shared;
		if(FPlatformString::Stricmp(InString, TEXT("All")) == 0)		return CST_All;
		return ReturnIfConversionFails;
	}

	static const TCHAR* ToString(const Type InType)
	{
		switch(InType)
		{
		case CST_System:
			return TEXT("System");
		case CST_Local:
			return TEXT("Local");
		case CST_Private:
			return TEXT("Private");
		case CST_Shared:
			return TEXT("Shared");
		case CST_All:
			return TEXT("All");
		default:
			break;
		}
		return TEXT("");
	}

	static FText ToText(const Type InType)
	{
		switch(InType)
		{
		case CST_System:
			return NSLOCTEXT("ECollectionShareType", "CST_System", "System");
		case CST_Local:
			return NSLOCTEXT("ECollectionShareType", "CST_Local", "Local");
		case CST_Private:
			return NSLOCTEXT("ECollectionShareType", "CST_Private", "Private");
		case CST_Shared:
			return NSLOCTEXT("ECollectionShareType", "CST_Shared", "Shared");
		case CST_All:
			return NSLOCTEXT("ECollectionShareType", "CST_All", "All");
		default:
			break;
		}
		return FText::GetEmpty();
	}

	static FText GetDescription(const Type InType)
	{
		switch (InType)
		{
		case CST_Local:
			return NSLOCTEXT("ECollectionShareType", "CST_Local_Description", "Local. This collection is only visible to you and is not in revision control.");
		case CST_Private:
			return NSLOCTEXT("ECollectionShareType", "CST_Private_Description", "Private. This collection is only visible to you.");
		case CST_Shared:
			return NSLOCTEXT("ECollectionShareType", "CST_Shared_Description", "Shared. This collection is visible to everyone.");
		case CST_System:
		case CST_All:
		default:
			break;
		}
		return FText::GetEmpty();
	}

	static FName GetIconStyleName(const Type InType, const TCHAR* InSizeSuffix = TEXT(".Small"))
	{
		switch (InType)
		{
		case CST_Local:
			return FName(*FString::Printf(TEXT("ContentBrowser.Local%s"), InSizeSuffix));
		case CST_Private:
			return FName(*FString::Printf(TEXT("ContentBrowser.Private%s"), InSizeSuffix));
		case CST_Shared:
			return FName(*FString::Printf(TEXT("ContentBrowser.Shared%s"), InSizeSuffix));
		case CST_System:
		case CST_All:
		default:
			break;
		}
		return NAME_None;
	}

	static bool IsValidChildType(const Type InParentType, const Type InChildType)
	{
		if (InParentType == CST_Local)
		{
			// Local collections can only contain other local collections
			return InChildType == CST_Local;
		}

		if (InParentType == CST_Private)
		{
			// Private collections can only contain local or private collections
			return InChildType == CST_Local || InChildType == CST_Private;
		}

		if (InParentType == CST_Shared)
		{
			// Shared collections can contain any kind of collection
			return InChildType == CST_Local || InChildType == CST_Private || InChildType == CST_Shared;
		}

		return false;
	}
};

ENUM_RANGE_BY_COUNT(ECollectionShareType::Type, ECollectionShareType::CST_All);

/** Controls how the collections manager will recurse when performing work against a given collection */
struct ECollectionRecursionFlags
{
	typedef uint32 Flags;
	enum Flag
	{
		/** Include the current collection when performing work */
		Self = 1<<0,
		/** Include the parent collections when performing work */
		Parents = 1<<1,
		/** Include the child collections when performing work */
		Children = 1<<2,

		/** Include parent collections in addition to the current collection */
		SelfAndParents = Self | Parents,
		/** Include child collections in addition to the current collection */
		SelfAndChildren = Self | Children,
		/** Include parent and child collections in addition to the current collection */
		All = Self | Parents | Children,
	};
};

/** A name/type pair to uniquely identify a collection in a collection container */
struct FCollectionNameType
{
	FCollectionNameType(FName InName, ECollectionShareType::Type InType)
		: Name(InName)
		, Type(InType)
	{}

	bool operator==(const FCollectionNameType& Other) const
	{
		return Name == Other.Name && Type == Other.Type;
	}

	bool operator!=(const FCollectionNameType& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash( const FCollectionNameType& Key )
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(Key.Name));
		Hash = HashCombine(Hash, GetTypeHash(Key.Type));
		return Hash;
	}

	FName Name;
	ECollectionShareType::Type Type;
};

/** A struct to uniquely identify a collection */
struct FCollectionRef
{
	FCollectionRef(const TSharedPtr<ICollectionContainer>& InContainer, const FCollectionNameType& InCollection)
		: Container(InContainer)
		, Name(InCollection.Name)
		, Type(InCollection.Type)
	{
	}

	FCollectionRef(const TSharedPtr<ICollectionContainer>& InContainer, FName InName, ECollectionShareType::Type InType)
		: Container(InContainer)
		, Name(InName)
		, Type(InType)
	{
	}

	bool operator==(const FCollectionRef& Other) const
	{
		return Container == Other.Container && Name == Other.Name && Type == Other.Type;
	}

	bool operator!=(const FCollectionRef& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FCollectionRef& Key)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(Key.Container));
		Hash = HashCombine(Hash, GetTypeHash(Key.Name));
		Hash = HashCombine(Hash, GetTypeHash(Key.Type));
		return Hash;
	}

	bool IsValid() const
	{
		return Container.IsValid() && Name != NAME_None && Type < ECollectionShareType::CST_All;
	}

	TSharedPtr<ICollectionContainer> Container;
	FName Name;
	ECollectionShareType::Type Type;
};

/** The status information for a collection */
struct FCollectionStatusInfo
{
	/** True if the collection has unsaved changes */
	bool bIsDirty;

	/** True if the collection is empty */
	bool bIsEmpty;

	/** True if the collection uses SCC */
	bool bUseSCC;

	/** Number of objects within the collection (static collections only) */
	int32 NumObjects;

	/** The current source control state of the collection. Only filled in for those collections that are under source control, but may also be empty if the SCC is unavailable (see bUseSCC) */
	FSourceControlStatePtr SCCState;
};

class ICollectionRedirectorFollower
{
public:
	virtual ~ICollectionRedirectorFollower() = default;

	/** Given an object path, will see if it needs to follow any redirectors, and if so, will populate OutNewObjectPath with the new name and return true */
	virtual bool FixupObject(const FSoftObjectPath& InObjectPath, FSoftObjectPath& OutNewObjectPath) = 0;
};
