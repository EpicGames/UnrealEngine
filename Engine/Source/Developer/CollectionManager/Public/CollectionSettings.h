// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "CollectionSettings.generated.h"

UCLASS(minimalapi, config=EditorSettings)
class UCollectionSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	FText GetSectionText() const override;
	FText GetSectionDescription() const override;
#endif
	
	/** When enabled, Shared and Private collections will automatically commit their changes to source control */
	UPROPERTY(config, EditAnywhere, Category=Collections, meta = (DisplayName="Auto-Commit to SourceControl on Save"))
	bool bAutoCommitOnSave = true;
};