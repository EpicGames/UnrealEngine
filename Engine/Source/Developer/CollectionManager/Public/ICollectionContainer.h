// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"

class ICollectionSource;
class ITextFilterExpressionContext;

class ICollectionContainer : public TSharedFromThis<ICollectionContainer>
{
public:
	ICollectionContainer() = default;
	ICollectionContainer(const ICollectionContainer&) = delete;
	virtual ~ICollectionContainer() = default;

	ICollectionContainer& operator=(const ICollectionContainer&) = delete;

	/** Returns the source of the collections in this container */
	virtual const TSharedRef<ICollectionSource>& GetCollectionSource() const = 0;

	/** Returns whether or not the collection container is read-only for the specified share type */
	virtual bool IsReadOnly(ECollectionShareType::Type ShareType) const = 0;

	/** Sets whether or not the collection container is read-only for the specified share type */
	virtual void SetReadOnly(ECollectionShareType::Type ShareType, bool bReadOnly) = 0;

	/** Returns whether or not the collection container should be hidden in the Editor's UI */
	virtual bool IsHidden() const = 0;

	/** Sets whether or not the collection container should be hidden in the Editor's UI */
	virtual void SetHidden(bool bHidden) = 0;

	/** Returns whether or not the collection container contains any collections */
	virtual bool HasCollections() const = 0;

	/** Returns the list of collections */
	virtual void GetCollections(TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of collections with the given name */
	virtual void GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of collection names of the specified share type */
	virtual void GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the list of root-level collections */
	virtual void GetRootCollections(TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of root-level collection names of the specified share type */
	virtual void GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the list of child collections of the given collection */
	virtual void GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of child collections of the given collection that are also of the specified share type */
	virtual void GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the parent collection of the given collection, or an unset value if there is no parent set */
	virtual TOptional<FCollectionNameType> GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const = 0;

	/** Returns true if the collection exists */
	virtual bool CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const = 0;

	/**
	 * Gets a list of assets in the given collection
	 * 
	 * @return true if any objects were found
	 */
	virtual bool GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& OutAssetPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/**
	 * Gets a list of class paths in the given collection
	 * 
	 * @return true if any objects were found
	 */
	virtual bool GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/**
	 * Gets a list of objects in the given collection
	 * 
	 * @return true if any objects were found
	 */
	virtual bool GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which the specified object exists of the specified share type */
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which the specified object exists */
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which any of the specified objects exist */
	virtual void GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a string containing a comma separated list of collections in which the specified object exists of the specified share type */
	virtual FString GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, bool bFullPaths = true) const = 0;

	/** Returns a path that uniquely identifies a collection */
	virtual FString MakeCollectionPath(FName CollectionName, ECollectionShareType::Type ShareType) const = 0;

	/** 
	 * Creates a unique collection name for the given type taking the form BaseName+(unique number) 
	 * 
	 * @warning This function is threadsafe but it does not secure the name from another thread creating the collection in between calls to the ICollectionContainer API.
	 */
	virtual void CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const = 0;

	/**
	 * Check whether the given name is valid for a new collection.
	 *
	 * @param CollectionName Potential collection name
	 * @param ShareType Type of collection to check for conflicts against. CST_All checks against all collcetion types.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true If a collection can be created with the name.
	 * @return false If the name contains invalid characters or conflicts with an existing collection of the same type.
	 */
	virtual bool IsValidCollectionName(
		const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) const = 0;

	/**
	 * Creates a new collection. A .collection file will be added to disk.
	 *
	 * @param CollectionName The name of the new collection
	 * @param ShareType The way the collection is shared.
	 * @param StorageMode How does this collection store its objects? (static or dynamic)
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the add was successful.
	 */
	virtual bool CreateCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode,
		FText* OutError = nullptr) = 0;

	/**
	 * Renames a collection. A .collection file will be added to disk and a .collection file will be removed.
	 *
	 * @param CurrentCollectionName The current name of the collection.
	 * @param CurrentShareType The current way the collection is shared.
	 * @param NewCollectionName The new name of the collection.
	 * @param NewShareType The new way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the rename was successful.
	 */
	virtual bool RenameCollection(
		FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName,
		ECollectionShareType::Type NewShareType, FText* OutError = nullptr) = 0;

	/**
	 * Re-parents a collection. The parent collection may be re-saved if it's too old to have a stable GUID.
	 *
	 * @param CollectionName The name of the collection to set the parent of.
	 * @param ShareType The way the collection is shared.
	 * @param ParentCollectionName The new parent of the collection, or None to have the collection become a root
	 * collection.
	 * @param ParentShareType The way the new parent collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the re-parent was successful.
	 */
	virtual bool ReparentCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName,
		ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) = 0;

	/**
	 * Removes a collection to the asset registry. A .collection file will be deleted from disk.
	 *
	 * @param CollectionName The name of the collection to remove.
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the remove was successful
	 */
	virtual bool DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Adds an asset to the specified collection.
	 *
	 * @param CollectionName The collection in which to add the asset
	 * @param ShareType The way the collection is shared.
	 * @param ObjectPath the ObjectPath of the asset to add.
	 * @param OutNumAdded if non-NULL, the number of objects successfully added to the collection
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the add was successful.
	 */
	virtual bool AddToCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath,
		FText* OutError = nullptr) = 0;
	virtual bool AddToCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths,
		int32* OutNumAdded = nullptr, FText* OutError = nullptr) = 0;

	/**
	 * Removes the asset from the specified collection.
	 *
	 * @param CollectionName The collection from which to remove the asset
	 * @param ShareType The way the collection is shared.
	 * @param ObjectPath the ObjectPath of the asset to remove.
	 * @param OutNumRemoved if non-NULL, the number of objects successfully removed from the collection
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the remove was successful.
	 */
	virtual bool RemoveFromCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath,
		FText* OutError = nullptr) = 0;
	virtual bool RemoveFromCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths,
		int32* OutNumRemoved = nullptr, FText* OutError = nullptr) = 0;

	/**
	 * Sets the dynamic query text for the specified collection. 
	 *
	 * @param CollectionName The collection to set the query on.
	 * @param ShareType The way the collection is shared.
	 * @param InQueryText The new query to set.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the set was successful.
	 */
	virtual bool SetDynamicQueryText(
		FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText,
		FText* OutError = nullptr) = 0;

	/**
	 * Gets the dynamic query text for the specified collection. 
	 *
	 * @param CollectionName The collection to get the query from.
	 * @param ShareType The way the collection is shared.
	 * @param OutQueryText Filled with the query text.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the get was successful.
	 */
	virtual bool GetDynamicQueryText(
		FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText,
		FText* OutError = nullptr) const = 0;

	/**
	 * Tests the dynamic query for the specified collection against the context provided.
	 *
	 * @param CollectionName The collection to get the query from.
	 * @param ShareType The way the collection is shared.
	 * @param InContext The context to test against.
	 * @param OutResult Filled with the result of the query.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the get was successful.
	 */
	virtual bool TestDynamicQuery(
		FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext,
		bool& OutResult, FText* OutError = nullptr) const = 0;

	/**
	 * Removes all assets from the specified collection.
	 *
	 * @param CollectionName The collection to empty
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the clear was successful.
	 */
	virtual bool EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Save the collection (if dirty) and check it into source control (if under SCC control)
	 *
	 * Note: Generally you won't need to save collections manually as the collection container takes
	 * care of that as objects and added/removed, etc. however, you may want to manually save a
	 * collection if a previous save attempt failed (and you've since corrected the issue), or if
	 * the collection contains redirected object references that you'd like to save to disk.
	 *
	 * @param CollectionName The collection to save
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the save was successful.
	 */
	virtual bool SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Update the collection to make sure it's using the latest version from source control (if
	 * under SCC control)
	 *
	 * Note: Generally you won't need to update collections manually as the collection container takes
	 * care of that as collections are saved to disk.
	 *
	 * @param CollectionName The collection to update
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the update was successful.
	 */
	virtual bool UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Gets the status info for the specified collection
	 *
	 * @param CollectionName The collection to get the status info for
	 * @param ShareType The way the collection is shared.
	 * @param OutStatusInfo The status info to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the status info was filled in.
	 */
	virtual bool GetCollectionStatusInfo(
		FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo,
		FText* OutError = nullptr) const = 0;

	/**
	 * Checks whether any collections have a custom color set, optionally retrieving the list of in-use colors
	 *
	 * @param OutColors The optional color array to populate.
	 * @return true if there are any custom colors in use, false otherwise.
	 */
	virtual bool HasCollectionColors(TArray<FLinearColor>* OutColors = nullptr) const = 0;

	/**
	 * Gets the optional color for the specified collection
	 *
	 * @param CollectionName The collection to get the color for
	 * @param ShareType The way the collection is shared.
	 * @param OutColor The optional color to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the collection was found, and the color was optionally filled in.
	 */
	virtual bool GetCollectionColor(
		FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor,
		FText* OutError = nullptr) const = 0;

	/**
	 * Sets the optional color for the specified collection
	 *
	 * @param CollectionName The collection to set the color for
	 * @param ShareType The way the collection is shared.
	 * @param NewColor The optional color to set.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the collection was found, and the color was set.
	 */
	virtual bool SetCollectionColor(
		FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor,
		FText* OutError = nullptr) = 0;

	/**
	 * Gets the method by which the specified collection stores its objects (static or dynamic)
	 *
	 * @param CollectionName The collection to get the storage mode for
	 * @param ShareType The way the collection is shared.
	 * @param OutStorageMode The variable to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the status info was filled in.
	 */
	virtual bool GetCollectionStorageMode(
		FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode,
		FText* OutError = nullptr) const = 0;

	/**
	 * Check to see if the given object exists in the given collection
	 *
	 * @param ObjectPath The object to test for its existence in the collection.
	 * @param CollectionName The collection to test.
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the object is in the collection.
	 */
	virtual bool IsObjectInCollection(
		const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType,
		ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self,
		FText* OutError = nullptr) const = 0;

	/**
	 * Check to see if the given collection is valid to be used as the parent of another collection.
	 * A collection may not be parented to itself, nor any of its current children.
	 *
	 * @param CollectionName The name of the collection to check the parent of.
	 * @param ShareType The way the collection is shared.
	 * @param ParentCollectionName The name of the parent collection to test against.
	 * @param ParentShareType The way the new parent collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the parent is considered valid for this collection.
	 */
	virtual bool IsValidParentCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName,
		ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) const = 0;

	/** Event for when the collection container's hidden state changes */
	DECLARE_EVENT_TwoParams(ICollectionContainer, FIsHiddenChangedEvent, ICollectionContainer&, bool);
	virtual FIsHiddenChangedEvent& OnIsHiddenChanged() = 0;

	/** Event for when collections are created */
	DECLARE_EVENT_TwoParams(ICollectionContainer, FCollectionCreatedEvent, ICollectionContainer&, const FCollectionNameType&);
	virtual FCollectionCreatedEvent& OnCollectionCreated() = 0;

	/** Event for when collections are destroyed */
	DECLARE_EVENT_TwoParams(ICollectionContainer, FCollectionDestroyedEvent, ICollectionContainer&, const FCollectionNameType&);
	virtual FCollectionDestroyedEvent& OnCollectionDestroyed() = 0;

	/** Event for when assets are added to a collection */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAssetsAddedToCollection, ICollectionContainer&, const FCollectionNameType&, TConstArrayView<FSoftObjectPath>);
	virtual FOnAssetsAddedToCollection& OnAssetsAddedToCollection() = 0;

	/** Event for when assets are removed from a collection */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAssetsRemovedFromCollection, ICollectionContainer&, const FCollectionNameType&, TConstArrayView<FSoftObjectPath>);
	virtual FOnAssetsRemovedFromCollection& OnAssetsRemovedFromCollection() = 0;

	/** Event for when collections are renamed */
	DECLARE_EVENT_ThreeParams(ICollectionContainer, FCollectionRenamedEvent, ICollectionContainer&, const FCollectionNameType&, const FCollectionNameType&);
	virtual FCollectionRenamedEvent& OnCollectionRenamed() = 0;

	/** Event for when collections are re-parented (params: Collection, OldParent, NewParent) */
	DECLARE_EVENT_FourParams(ICollectionContainer, FCollectionReparentedEvent, ICollectionContainer&, const FCollectionNameType&, const TOptional<FCollectionNameType>&, const TOptional<FCollectionNameType>&);
	virtual FCollectionReparentedEvent& OnCollectionReparented() = 0;

	/** Event for when collections is updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	DECLARE_EVENT_TwoParams(ICollectionContainer, FCollectionUpdatedEvent, ICollectionContainer&, const FCollectionNameType&);
	virtual FCollectionUpdatedEvent& OnCollectionUpdated() = 0;
};
