// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"

class ICollectionContainer;
class ICollectionSource;
class ITextFilterExpressionContext;

class ICollectionManager
{
public:
	ICollectionManager() = default;
	ICollectionManager(const ICollectionManager&) = delete;
	virtual ~ICollectionManager() = default;

	ICollectionManager& operator=(const ICollectionManager&) = delete;

	/** Returns the collection container for the current uproject */
	virtual const TSharedRef<ICollectionContainer>& GetProjectCollectionContainer() const = 0;

	/** Creates and returns a new collection container for the specified collection source */
	virtual TSharedPtr<ICollectionContainer> AddCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource) = 0;

	/** Removes the specified collection container */
	virtual bool RemoveCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer) = 0;

	/** Returns true if this collection manager contains the specified collection container */
	virtual bool HasCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer) const = 0;

	/** Returns the collection container that matches the specified collection source name, if any */
	virtual TSharedPtr<ICollectionContainer> FindCollectionContainer(FName CollectionSourceName) const = 0;

	/** Returns the collection container that matches the specified collection source, if any */
	virtual TSharedPtr<ICollectionContainer> FindCollectionContainer(const TSharedRef<ICollectionSource>& CollectionSource) const = 0;

	/** Gets all collection containers */
	virtual void GetCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const = 0;

	/** Gets all collection containers that aren't hidden */
	virtual void GetVisibleCollectionContainers(TArray<TSharedPtr<ICollectionContainer>>& OutCollectionContainers) const = 0;

	/** Returns true if the collection path could be parsed into its constituent parts, and the collection container exists (if an out parameter for it was passed in) */
	virtual bool TryParseCollectionPath(const FString& CollectionPath, TSharedPtr<ICollectionContainer>* OutCollectionContainer, FName* OutCollectionName, ECollectionShareType::Type* OutShareType) const = 0;

	/** Returns whether or not the collection manager contains any collections */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->HasCollections() to disambiguate which collection container to use.")
	virtual bool HasCollections() const = 0;

	/** Returns the list of collections */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollections(...) to disambiguate which collection container to use.")
	virtual void GetCollections(TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of collections with the given name */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollections(...) to disambiguate which collection container to use.")
	virtual void GetCollections(FName CollectionName, TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of collection names of the specified share type */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionNames(...) to disambiguate which collection container to use.")
	virtual void GetCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the list of root-level collections */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetRootCollections(...) to disambiguate which collection container to use.")
	virtual void GetRootCollections(TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of root-level collection names of the specified share type */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetRootCollectionNames(...) to disambiguate which collection container to use.")
	virtual void GetRootCollectionNames(ECollectionShareType::Type ShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the list of child collections of the given collection */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetChildCollections(...) to disambiguate which collection container to use.")
	virtual void GetChildCollections(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FCollectionNameType>& OutCollections) const = 0;

	/** Returns the list of child collections of the given collection that are also of the specified share type */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetChildCollectionNames(...) to disambiguate which collection container to use.")
	virtual void GetChildCollectionNames(FName CollectionName, ECollectionShareType::Type ShareType, ECollectionShareType::Type ChildShareType, TArray<FName>& CollectionNames) const = 0;

	/** Returns the parent collection of the given collection, or an unset value if there is no parent set */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetParentCollection(...) to disambiguate which collection container to use.")
	virtual TOptional<FCollectionNameType> GetParentCollection(FName CollectionName, ECollectionShareType::Type ShareType) const = 0;

	/** Returns true if the collection exists */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->CollectionExists(...) to disambiguate which collection container to use.")
	virtual bool CollectionExists(FName CollectionName, ECollectionShareType::Type ShareType) const = 0;

	/**
	 * Gets a list of assets in the given collection
	 * 
	 * @return true if any objects were found
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetAssetsInCollection(...) to disambiguate which collection container to use.")
	virtual bool GetAssetsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& OutAssetPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/**
	 * Gets a list of class paths in the given collection
	 * 
	 * @return true if any objects were found
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetClassesInCollection(...) to disambiguate which collection container to use.")
	virtual bool GetClassesInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FTopLevelAssetPath>& ClassPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/**
	 * Gets a list of objects in the given collection
	 * 
	 * @return true if any objects were found
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetObjectsInCollection(...) to disambiguate which collection container to use.")
	virtual bool GetObjectsInCollection(FName CollectionName, ECollectionShareType::Type ShareType, TArray<FSoftObjectPath>& ObjectPaths, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which the specified object exists of the specified share type */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionsContainingObject(...) to disambiguate which collection container to use.")
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, TArray<FName>& OutCollectionNames, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which the specified object exists */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionsContainingObject(...) to disambiguate which collection container to use.")
	virtual void GetCollectionsContainingObject(const FSoftObjectPath& ObjectPath, TArray<FCollectionNameType>& OutCollections, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a list of collections in which any of the specified objects exist */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionsContainingObjects(...) to disambiguate which collection container to use.")
	virtual void GetCollectionsContainingObjects(const TArray<FSoftObjectPath>& ObjectPaths, TMap<FCollectionNameType, TArray<FSoftObjectPath>>& OutCollectionsAndMatchedObjects, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self) const = 0;

	/** Returns a string containing a comma separated list of collections in which the specified object exists of the specified share type */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionsStringForObject(...) to disambiguate which collection container to use.")
	virtual FString GetCollectionsStringForObject(const FSoftObjectPath& ObjectPath, ECollectionShareType::Type ShareType, ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self, bool bFullPaths = true) const = 0;

	/** 
	 * Creates a unique collection name for the given type taking the form BaseName+(unique number) 
	 * 
	 * @warning This function is threadsafe but it does not secure the name from another thread creating the collection in between calls to the ICollectionManager API.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->CreateUniqueCollectionName(...) to disambiguate which collection container to use.")
	virtual void CreateUniqueCollectionName(FName BaseName, ECollectionShareType::Type ShareType, FName& OutCollectionName) const = 0;

	/**
	 * Check whether the given name is valid for a new collection.
	 *
	 * @param CollectionName Potential collection name
	 * @param ShareType Type of collection to check for conflicts against. CST_All checks against all collcetion types.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true If a collection can be created with the name.
	 * @return false If the name contains invalid characters or conflicts with an existing collection of the same type.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->IsValidCollectionName(...) to disambiguate which collection container to use.")
	virtual bool IsValidCollectionName(
		const FString& CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) const = 0;

	/**
	 * Creates a new collection. A .collection file will be added to disk.
	 *
	 * @param CollectionName The name of the new collection
	 * @param ShareType The way the collection is shared.
	 * @param StorageMode How does this collection store its objects? (static or dynamic)
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the add was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->CreateCollection(...) to disambiguate which collection container to use.")
	virtual bool CreateCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type StorageMode,
		FText* OutError = nullptr) = 0;

	/**
	 * Renames a collection. A .collection file will be added to disk and a .collection file will be removed.
	 *
	 * @param CurrentCollectionName The current name of the collection.
	 * @param CurrentShareType The current way the collection is shared.
	 * @param NewCollectionName The new name of the collection.
	 * @param NewShareType The new way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the rename was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->RenameCollection(...) to disambiguate which collection container to use.")
	virtual bool RenameCollection(
		FName CurrentCollectionName, ECollectionShareType::Type CurrentShareType, FName NewCollectionName,
		ECollectionShareType::Type NewShareType, FText* OutError = nullptr) = 0;

	/**
	 * Re-parents a collection. The parent collection may be re-saved if it's too old to have a stable GUID.
	 *
	 * @param CollectionName The name of the collection to set the parent of.
	 * @param ShareType The way the collection is shared.
	 * @param ParentCollectionName The new parent of the collection, or None to have the collection become a root
	 * collection.
	 * @param ParentShareType The way the new parent collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the re-parent was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->ReparentCollection(...) to disambiguate which collection container to use.")
	virtual bool ReparentCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName,
		ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) = 0;

	/**
	 * Removes a collection to the asset registry. A .collection file will be deleted from disk.
	 *
	 * @param CollectionName The name of the collection to remove.
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the remove was successful
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->DestroyCollection(...) to disambiguate which collection container to use.")
	virtual bool DestroyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Adds an asset to the specified collection.
	 *
	 * @param CollectionName The collection in which to add the asset
	 * @param ShareType The way the collection is shared.
	 * @param ObjectPath the ObjectPath of the asset to add.
	 * @param OutNumAdded if non-NULL, the number of objects successfully added to the collection
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the add was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->AddToCollection(...) to disambiguate which collection container to use.")
	virtual bool AddToCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath,
		FText* OutError = nullptr) = 0;
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->AddToCollection(...) to disambiguate which collection container to use.")
	virtual bool AddToCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths,
		int32* OutNumAdded = nullptr, FText* OutError = nullptr) = 0;

	/**
	 * Removes the asset from the specified collection.
	 *
	 * @param CollectionName The collection from which to remove the asset
	 * @param ShareType The way the collection is shared.
	 * @param ObjectPath the ObjectPath of the asset to remove.
	 * @param OutNumRemoved if non-NULL, the number of objects successfully removed from the collection
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the remove was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->RemoveFromCollection(...) to disambiguate which collection container to use.")
	virtual bool RemoveFromCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, const FSoftObjectPath& ObjectPath,
		FText* OutError = nullptr) = 0;
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->RemoveFromCollection(...) to disambiguate which collection container to use.")
	virtual bool RemoveFromCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, TConstArrayView<FSoftObjectPath> ObjectPaths,
		int32* OutNumRemoved = nullptr, FText* OutError = nullptr) = 0;

	/**
	 * Sets the dynamic query text for the specified collection. 
	 *
	 * @param CollectionName The collection to set the query on.
	 * @param ShareType The way the collection is shared.
	 * @param InQueryText The new query to set.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the set was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->SetDynamicQueryText(...) to disambiguate which collection container to use.")
	virtual bool SetDynamicQueryText(
		FName CollectionName, ECollectionShareType::Type ShareType, const FString& InQueryText,
		FText* OutError = nullptr) = 0;

	/**
	 * Gets the dynamic query text for the specified collection. 
	 *
	 * @param CollectionName The collection to get the query from.
	 * @param ShareType The way the collection is shared.
	 * @param OutQueryText Filled with the query text.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the get was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetDynamicQueryText(...) to disambiguate which collection container to use.")
	virtual bool GetDynamicQueryText(
		FName CollectionName, ECollectionShareType::Type ShareType, FString& OutQueryText,
		FText* OutError = nullptr) const = 0;

	/**
	 * Tests the dynamic query for the specified collection against the context provided.
	 *
	 * @param CollectionName The collection to get the query from.
	 * @param ShareType The way the collection is shared.
	 * @param InContext The context to test against.
	 * @param OutResult Filled with the result of the query.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the get was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->TestDynamicQuery(...) to disambiguate which collection container to use.")
	virtual bool TestDynamicQuery(
		FName CollectionName, ECollectionShareType::Type ShareType, const ITextFilterExpressionContext& InContext,
		bool& OutResult, FText* OutError = nullptr) const = 0;

	/**
	 * Removes all assets from the specified collection.
	 *
	 * @param CollectionName The collection to empty
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the clear was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->EmptyCollection(...) to disambiguate which collection container to use.")
	virtual bool EmptyCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Save the collection (if dirty) and check it into source control (if under SCC control)
	 *
	 * Note: Generally you won't need to save collections manually as the collection manager takes
	 * care of that as objects and added/removed, etc. however, you may want to manually save a
	 * collection if a previous save attempt failed (and you've since corrected the issue), or if
	 * the collection contains redirected object references that you'd like to save to disk.
	 *
	 * @param CollectionName The collection to save
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the save was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->SaveCollection(...) to disambiguate which collection container to use.")
	virtual bool SaveCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Update the collection to make sure it's using the latest version from source control (if
	 * under SCC control)
	 *
	 * Note: Generally you won't need to update collections manually as the collection manager takes
	 * care of that as collections are saved to disk.
	 *
	 * @param CollectionName The collection to update
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the update was successful.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->UpdateCollection(...) to disambiguate which collection container to use.")
	virtual bool UpdateCollection(FName CollectionName, ECollectionShareType::Type ShareType, FText* OutError = nullptr) = 0;

	/**
	 * Gets the status info for the specified collection
	 *
	 * @param CollectionName The collection to get the status info for
	 * @param ShareType The way the collection is shared.
	 * @param OutStatusInfo The status info to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the status info was filled in.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionStatusInfo(...) to disambiguate which collection container to use.")
	virtual bool GetCollectionStatusInfo(
		FName CollectionName, ECollectionShareType::Type ShareType, FCollectionStatusInfo& OutStatusInfo,
		FText* OutError = nullptr) const = 0;

	/**
	 * Checks whether any collections have a custom color set, optionally retrieving the list of in-use colors
	 *
	 * @param OutColors The optional color array to populate.
	 * @return true if there are any custom colors in use, false otherwise.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->HasCollectionColors(...) to disambiguate which collection container to use.")
	virtual bool HasCollectionColors(TArray<FLinearColor>* OutColors = nullptr) const = 0;

	/**
	 * Gets the optional color for the specified collection
	 *
	 * @param CollectionName The collection to get the color for
	 * @param ShareType The way the collection is shared.
	 * @param OutColor The optional color to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the collection was found, and the color was optionally filled in.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionColor(...) to disambiguate which collection container to use.")
	virtual bool GetCollectionColor(
		FName CollectionName, ECollectionShareType::Type ShareType, TOptional<FLinearColor>& OutColor,
		FText* OutError = nullptr) const = 0;

	/**
	 * Sets the optional color for the specified collection
	 *
	 * @param CollectionName The collection to set the color for
	 * @param ShareType The way the collection is shared.
	 * @param NewColor The optional color to set.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the collection was found, and the color was set.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->SetCollectionColor(...) to disambiguate which collection container to use.")
	virtual bool SetCollectionColor(
		FName CollectionName, ECollectionShareType::Type ShareType, const TOptional<FLinearColor>& NewColor,
		FText* OutError = nullptr) = 0;

	/**
	 * Gets the method by which the specified collection stores its objects (static or dynamic)
	 *
	 * @param CollectionName The collection to get the storage mode for
	 * @param ShareType The way the collection is shared.
	 * @param OutStorageMode The variable to populate.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the status info was filled in.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->GetCollectionStorageMode(...) to disambiguate which collection container to use.")
	virtual bool GetCollectionStorageMode(
		FName CollectionName, ECollectionShareType::Type ShareType, ECollectionStorageMode::Type& OutStorageMode,
		FText* OutError = nullptr) const = 0;

	/**
	 * Check to see if the given object exists in the given collection
	 *
	 * @param ObjectPath The object to test for its existence in the collection.
	 * @param CollectionName The collection to test.
	 * @param ShareType The way the collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the object is in the collection.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->IsObjectInCollection(...) to disambiguate which collection container to use.")
	virtual bool IsObjectInCollection(
		const FSoftObjectPath& ObjectPath, FName CollectionName, ECollectionShareType::Type ShareType,
		ECollectionRecursionFlags::Flags RecursionMode = ECollectionRecursionFlags::Self,
		FText* OutError = nullptr) const = 0;

	/**
	 * Check to see if the given collection is valid to be used as the parent of another collection.
	 * A collection may not be parented to itself, nor any of its current children.
	 *
	 * @param CollectionName The name of the collection to check the parent of.
	 * @param ShareType The way the collection is shared.
	 * @param ParentCollectionName The name of the parent collection to test against.
	 * @param ParentShareType The way the new parent collection is shared.
	 * @param OutError Optional human readable explanation of why the operation failed
	 * @return true if the parent is considered valid for this collection.
	 */
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->IsValidParentCollection(...) to disambiguate which collection container to use.")
	virtual bool IsValidParentCollection(
		FName CollectionName, ECollectionShareType::Type ShareType, FName ParentCollectionName,
		ECollectionShareType::Type ParentShareType, FText* OutError = nullptr) const = 0;

	/** Returns the most recent error. */
	UE_DEPRECATED(5.5, "Deprecated for thread safety reasons. Functions which populated this value now take an optional FText* out parameter.")
	virtual FText GetLastError() const = 0;

	/**
	 * Called to notify the collections that they should fix-up their object references so that they no longer contain any redirectors
	 * References are only updated in-memory, and won't be saved to disk until a redirector is deleted (which forces our hand), or the collection is saved for any other reason
	 */
	virtual void HandleFixupRedirectors(ICollectionRedirectorFollower& InRedirectorFollower) = 0;

	/**
	 * Called to notify the collections that a redirector has been deleted and that they should ensure their on-disk representation is re-saved with the fixed up in-memory version
	 * @return true if all of the collections that were referencing this redirector could be re-saved, false otherwise
	 */
	virtual bool HandleRedirectorDeleted(const FSoftObjectPath& ObjectPath, FText* OutError = nullptr) = 0;

	/**
	 * Called to notify the collections that redirectors have been deleted and that they should ensure their on-disk representation is re-saved with the fixed up in-memory version
	 * @return true if all of the collections that were referencing these redirectors could be re-saved, false otherwise
	 */
	virtual bool HandleRedirectorsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths, FText* OutError = nullptr) = 0;

	/** Called to notify the collections that an object has been renamed or moved */
	virtual void HandleObjectRenamed(const FSoftObjectPath& OldObjectPath, const FSoftObjectPath& NewObjectPath) = 0;

	/** Called to notify the collections that an object has been deleted */
	virtual void HandleObjectDeleted(const FSoftObjectPath& ObjectPath) = 0;

	/** Called to notify the collections that objects have been deleted */
	virtual void HandleObjectsDeleted(TConstArrayView<FSoftObjectPath> ObjectPaths) = 0;

	/** Event for when collection containers are created */
	DECLARE_EVENT_OneParam(ICollectionManager, FCollectionContainerCreatedEvent, const TSharedRef<ICollectionContainer>&);
	virtual FCollectionContainerCreatedEvent& OnCollectionContainerCreated() = 0;

	/** Event for when collection containers are destroyed */
	DECLARE_EVENT_OneParam(ICollectionManager, FCollectionContainerDestroyedEvent, const TSharedRef<ICollectionContainer>&);
	virtual FCollectionContainerDestroyedEvent& OnCollectionContainerDestroyed() = 0;

	/** Event for when collections are created */
	DECLARE_EVENT_OneParam( ICollectionManager, FCollectionCreatedEvent, const FCollectionNameType& );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnCollectionCreated() to disambiguate which collection container to use.")
	virtual FCollectionCreatedEvent& OnCollectionCreated() = 0;

	/** Event for when collections are destroyed */
	DECLARE_EVENT_OneParam( ICollectionManager, FCollectionDestroyedEvent, const FCollectionNameType& );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnCollectionDestroyed() to disambiguate which collection container to use.")
	virtual FCollectionDestroyedEvent& OnCollectionDestroyed() = 0;

	/** Event for when assets are added to a collection */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetsAddedToCollection, const FCollectionNameType&, TConstArrayView<FSoftObjectPath> );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnAssetsAddedToCollection() to disambiguate which collection container to use.")
	virtual FOnAssetsAddedToCollection& OnAssetsAddedToCollection() = 0;

	/** Event for when assets are removed from a collection */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnAssetsRemovedFromCollection, const FCollectionNameType&, TConstArrayView<FSoftObjectPath> );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnAssetsRemovedFromCollection() to disambiguate which collection container to use.")
	virtual FOnAssetsRemovedFromCollection& OnAssetsRemovedFromCollection() = 0;

	/** Event for when collections are renamed */
	DECLARE_EVENT_TwoParams( ICollectionManager, FCollectionRenamedEvent, const FCollectionNameType&, const FCollectionNameType& );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnCollectionRenamed() to disambiguate which collection container to use.")
	virtual FCollectionRenamedEvent& OnCollectionRenamed() = 0;

	/** Event for when collections are re-parented (params: Collection, OldParent, NewParent) */
	DECLARE_EVENT_ThreeParams( ICollectionManager, FCollectionReparentedEvent, const FCollectionNameType&, const TOptional<FCollectionNameType>&, const TOptional<FCollectionNameType>& );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnCollectionReparented() to disambiguate which collection container to use.")
	virtual FCollectionReparentedEvent& OnCollectionReparented() = 0;

	/** Event for when collections is updated, or otherwise changed and we can't tell exactly how (eg, after updating from source control and merging) */
	DECLARE_EVENT_OneParam( ICollectionManager, FCollectionUpdatedEvent, const FCollectionNameType& );
	UE_DEPRECATED(5.6, "ICollectionContainer has been introduced as a new top level object for housing collections, substitute wtih GetProjectCollectionContainer()->OnCollectionUpdated() to disambiguate which collection container to use.")
	virtual FCollectionUpdatedEvent& OnCollectionUpdated() = 0;

	/** 
	 * When a collection checkin happens, use this event to add additional text to the changelist description.
	 * Delegates bound to this event must be safe to call on any thread. 
	 */
	DECLARE_EVENT_TwoParams( ICollectionManager, FAddToCollectionCheckinDescriptionEvent, const FName& /*CollectionName*/, TArray<FText>& /* OutAdditionalLines*/);
	virtual FAddToCollectionCheckinDescriptionEvent& OnAddToCollectionCheckinDescriptionEvent() = 0;

	/** Internal API for processing HandleObjectsDeleted calls in a batch-friendly way */
	UE_INTERNAL virtual void SuppressObjectDeletionHandling() = 0;
	UE_INTERNAL virtual void ResumeObjectDeletionHandling() = 0;
};
