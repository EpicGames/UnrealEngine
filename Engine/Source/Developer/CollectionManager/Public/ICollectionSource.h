// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"

class ICollectionSource
{
public:
	ICollectionSource() = default;
	ICollectionSource(const ICollectionSource&) = delete;
	virtual ~ICollectionSource() = default;

	ICollectionSource& operator=(const ICollectionSource&) = delete;

	/** Returns a name that identifies this collection source. */
	virtual FName GetName() const = 0;

	/** Returns a friendly name to display to the user for this collection source. */
	virtual FText GetTitle() const = 0;

	/** Returns the folder path used to store the collections of a given share type. */
	virtual const FString& GetCollectionFolder(const ECollectionShareType::Type InCollectionShareType) const = 0;

	/** Returns the file path of the config file used to store settings related to this collection source. */
	virtual FString GetEditorPerProjectIni() const = 0;

	/** Returns the file path to a file that can be used to check the source control status of collections in this collection source. */
	virtual FString GetSourceControlStatusHintFilename() const = 0;

	/** Returns lines to add to the description when checking in changes to a collection to source control. */
	virtual TArray<FText> GetSourceControlCheckInDescription(FName CollectionName) const = 0;
};
